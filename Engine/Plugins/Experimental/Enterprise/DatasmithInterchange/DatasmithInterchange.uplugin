{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Interchange Datasmith",
	"Description": "Interchange Importer for Datasmith.",
	"Category": "Importers",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"PlatformAllowList": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "DatasmithInterchange",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DatasmithImporter",
			"Enabled": true
		},
		{
			"Name": "DatasmithContent",
			"Enabled": true
		},
		{
			"Name": "DatasmithCADImporter",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64"
			]
		},
		{
			"Name": "DatasmithC4DImporter",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Mac"
			]
		},
		{
			"Name": "Interchange",
			"Enabled": true
		}
	]
}