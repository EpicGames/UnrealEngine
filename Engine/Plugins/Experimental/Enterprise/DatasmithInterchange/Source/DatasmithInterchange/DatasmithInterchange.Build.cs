// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{	public class DatasmithInterchange : ModuleRules
	{
		public DatasmithInterchange(ReadOnlyTargetRules Target)
			: base(Target)
		{

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithContent",
					"DatasmithCore",
					"DatasmithTranslator",
					"Engine",
					"ExternalSource",
					"InterchangeEngine",
					"InterchangeNodes",
					"InterchangeCommonParser",
					"InterchangeFactoryNodes",
					"InterchangeImport",
					"InterchangePipelines",
					"MeshDescription",
					"StaticMeshDescription"
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DatasmithImporter",
						"DesktopPlatform",
						"InputCore",
						"InterchangeEditorPipelines",
						"PropertyEditor",
						"Slate",
						"SlateCore",
						"ToolMenus",
						"UnrealEd",
					}
				);
			}
			
			if(Target.LinkType != TargetLinkType.Monolithic)
			{
				PrivateDefinitions.Add("NO_MONOLITHIC_BUILD");
				PrivateDependencyModuleNames.Add("CADTools");
				PrivateDependencyModuleNames.Add("ParametricSurface");
			}

			PublicDependencyModuleNames.AddRange(
					new string[]
					{
					"Core",
					"CoreUObject",
					"InterchangeCore",
					"InterchangeImport",
					}
				);
		}
	}
}