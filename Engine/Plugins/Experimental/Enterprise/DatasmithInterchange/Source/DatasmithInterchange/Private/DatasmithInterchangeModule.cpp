// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithInterchangeModule.h"

#include "Engine/Blueprint.h"
#include "InterchangeDatasmithAreaLightFactory.h"
#include "InterchangeDatasmithCustomizations.h"
#include "InterchangeDatasmithLog.h"
#include "InterchangeDatasmithPipeline.h"
#include "InterchangeDatasmithTranslator.h"

#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "InterchangeReferenceMaterials/DatasmithC4DMaterialSelector.h"
#include "InterchangeReferenceMaterials/DatasmithCityEngineMaterialSelector.h"
#include "InterchangeReferenceMaterials/DatasmithRevitMaterialSelector.h"
#include "InterchangeReferenceMaterials/DatasmithSketchupMaterialSelector.h"
#include "InterchangeReferenceMaterials/DatasmithStdMaterialSelector.h"

#include "DatasmithTranslatorManager.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeGenericScenesPipeline.h"
#include "InterchangeManager.h"
#include "InterchangeProjectSettings.h"

#include "Logging/LogMacros.h"
#include "UObject/SoftObjectPath.h"

#if WITH_EDITOR
#include "PropertyEditorModule.h"
#endif

#define LOCTEXT_NAMESPACE "DatasmithInterchange"


DEFINE_LOG_CATEGORY(LogInterchangeDatasmith);

class FDatasmithInterchangeModule : public IDatasmithInterchangeModule
{
public:
	virtual void StartupModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FDatasmithInterchangeModule::OnPostEngineInit);
	}

	virtual void ShutdownModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

		UE::DatasmithInterchange::FDatasmithReferenceMaterialManager::Destroy();
		
#if WITH_EDITOR
		FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyEditorModule)
		{
			for (FName ClassName : ClassesToUnregisterOnShutdown)
			{
				PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
			}
		}
		ClassesToUnregisterOnShutdown.Empty();
#endif
	}

	void OnPostEngineInit()
	{
		using namespace UE::DatasmithInterchange;

		// Load the blueprint asset into memory while wew're on the game thread so that GetAreaLightActorBPClass() can safely be called from other threads.
		UBlueprint* AreaLightBlueprint = Cast<UBlueprint>(FSoftObjectPath(TEXT("/DatasmithContent/Datasmith/DatasmithArealight.DatasmithArealight")).TryLoad());
		//ensure(AreaLightBlueprint != nullptr);

		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		InterchangeManager.RegisterTranslator(UInterchangeDatasmithTranslator::StaticClass());
		InterchangeManager.RegisterFactory(UInterchangeDatasmithAreaLightFactory::StaticClass());

		// Add Datasmith translator to the Interchange engine
		FInterchangeTranslatorPipelines TranslatorPipeplines;
		TranslatorPipeplines.Translator = TSoftClassPtr<UInterchangeTranslatorBase>(UInterchangeDatasmithTranslator::StaticClass());
		TranslatorPipeplines.Pipelines.Add(FSoftObjectPath(TEXT("/DatasmithInterchange/InterchangeDatasmithDefault.InterchangeDatasmithDefault")));
		{
			FInterchangeImportSettings& ImportSettings = FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(false);

			const FName DefaultPipelineStack = ImportSettings.DefaultPipelineStack;
			if (ImportSettings.PipelineStacks.Contains(DefaultPipelineStack))
			{
				FInterchangePipelineStack& PipelineStack = ImportSettings.PipelineStacks[DefaultPipelineStack];
				PipelineStack.PerTranslatorPipelines.Add(TranslatorPipeplines);

				UE_LOGF(LogInterchangeDatasmith, Log, "InterchangeDatasmith's pipelines added to default Asset pipeline stack, %ls", *DefaultPipelineStack.ToString());
			}
			else
			{
				UE_LOGF(LogInterchangeDatasmith, Warning, "Cannot find Interchange default Asset's stack to add pipelines. InterchangeDatasmith asset import may not work properly.");
			}
		}

		{
			FInterchangeImportSettings& ImportSettings = FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(true);

			const FName DefaultPipelineStack = ImportSettings.DefaultPipelineStack;
			if (ImportSettings.PipelineStacks.Contains(DefaultPipelineStack))
			{
				FInterchangePipelineStack& PipelineStack = ImportSettings.PipelineStacks[DefaultPipelineStack];
				PipelineStack.PerTranslatorPipelines.Add(TranslatorPipeplines);

				UE_LOGF(LogInterchangeDatasmith, Log, "InterchangeDatasmith's pipelines added to default Scene pipeline stack, %ls", *DefaultPipelineStack.ToString());
			}
			else
			{
				UE_LOGF(LogInterchangeDatasmith, Warning, "Cannot find Interchange default Scene's stack to add pipelines. InterchangeDatasmith level import may not work properly.");
			}
		}

		FDatasmithReferenceMaterialManager::Create();

		//A minimal set of natively supported reference materials
		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("C4D"), MakeShared< FDatasmithC4DMaterialSelector >());
		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("Revit"), MakeShared< FDatasmithRevitMaterialSelector >());
		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("SketchUp"), MakeShared< FDatasmithSketchUpMaterialSelector >());
		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("CityEngine"), MakeShared< FDatasmithCityEngineMaterialSelector >());
		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("StdMaterial"), MakeShared< FDatasmithStdMaterialSelector >());

#if WITH_EDITOR
		ClassesToUnregisterOnShutdown.Reset();
		// Register details customizations
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		ClassesToUnregisterOnShutdown.Add(UInterchangeDatasmithTranslatorSettings::StaticClass()->GetFName());
		PropertyEditorModule.RegisterCustomClassLayout(ClassesToUnregisterOnShutdown.Last(), FOnGetDetailCustomizationInstance::CreateStatic(&FInterchangeDatasmithTranslatorSettingsCustomization::MakeInstance));
#endif
	}

#if WITH_EDITOR
	TArray<FName> ClassesToUnregisterOnShutdown;
#endif
};

IMPLEMENT_MODULE(FDatasmithInterchangeModule, DatasmithInterchange);

#undef LOCTEXT_NAMESPACE