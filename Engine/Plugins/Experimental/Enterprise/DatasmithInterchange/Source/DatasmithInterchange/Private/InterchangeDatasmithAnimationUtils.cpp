// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithLog.h"
#include "InterchangeDatasmithUtils.h"

#include "DatasmithAnimationElements.h"

#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeAnimSequenceFactoryNode.h"
#include "InterchangeAnimationDefinitions.h"

#include "Curves/RichCurve.h"
#include "Misc/FrameRate.h"

#define LOCTEXT_NAMESPACE "InterchangeDatasmithAnimationUtils"

namespace UE::DatasmithInterchange::AnimUtils
{
	class FLevelSequenceImportHelper
	{
	public:
		using FLevelSequencePtr = TSharedPtr<IDatasmithLevelSequenceElement>;

		FLevelSequenceImportHelper(UInterchangeBaseNodeContainer& InBaseNodeContainer)
			: BaseNodeContainer(InBaseNodeContainer)
		{}

		void Sort(TArray<FLevelSequencePtr>& LevelSequences);
		void ProcessLevelSequence(FLevelSequencePtr& InLevelSequenceElement, UInterchangeAnimationTrackSetNode* InLevelSequenceNode);

		TMap<FString, UE::DatasmithInterchange::AnimUtils::FAnimationPayloadDesc> PayLoadsMap;

	private:
		void ProcessSubsequenceTrack(IDatasmithSubsequenceAnimationElement& SubsequenceAnimation);
		FString ProcessVisibilityTrack(IDatasmithVisibilityAnimationElement& VisibilityAnimation, int32 TrackIndex);
		FString ProcessTransformTrack(IDatasmithTransformAnimationElement& TransformAnimation, int32 TrackIndex);

		void AddLevelSequence(FLevelSequencePtr& LevelSequence, TArray<FLevelSequencePtr>& SortedLevelSequences, TSet<FLevelSequencePtr>& Processed);

		template<class T>
		T* AddTrackNode(int32 TrackIndex)
		{
			if (T* TrackNode = NewObject<T>(&BaseNodeContainer))
			{
				const FString TrackName = LevelSequenceNode->GetDisplayLabel() + TEXT("_Track_") + FString::FromInt(TrackIndex);
				const FString TrackNodeUid = NodeUtils::LevelSequencePrefix + TrackName;
				BaseNodeContainer.SetupNode(TrackNode, TrackNodeUid, TrackName, EInterchangeNodeContainerType::TranslatedScene);

				return TrackNode;
			}

			return nullptr;
		}

	private:
		UInterchangeBaseNodeContainer& BaseNodeContainer;
		FLevelSequencePtr LevelSequenceElement;
		UInterchangeAnimationTrackSetNode* LevelSequenceNode = nullptr;
		TMap<FLevelSequencePtr, FString> TranslatedLevelSequences;
	};

	void FLevelSequenceImportHelper::ProcessLevelSequence(FLevelSequencePtr& InLevelSequenceElement, UInterchangeAnimationTrackSetNode* InLevelSequenceNode)
	{
		LevelSequenceElement = InLevelSequenceElement;
		LevelSequenceNode = InLevelSequenceNode;

		const float FrameRate = LevelSequenceElement->GetFrameRate();
		const int32 NumAnimations = LevelSequenceElement->GetAnimationsCount();

		for (int32 TrackIndex = 0; TrackIndex < NumAnimations; ++TrackIndex)
		{
			TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement = LevelSequenceElement->GetAnimation(TrackIndex);
			if (!AnimationElement)
			{
				continue;
			}

			FString PayLoadKey;

			if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
			{
				PayLoadKey = ProcessTransformTrack(static_cast<IDatasmithTransformAnimationElement&>(*AnimationElement), TrackIndex);
			}
			else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::VisibilityAnimation))
			{
				PayLoadKey = ProcessVisibilityTrack(static_cast<IDatasmithVisibilityAnimationElement&>(*AnimationElement), TrackIndex);
			}
			else if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::SubsequenceAnimation))
			{
				ProcessSubsequenceTrack(static_cast<IDatasmithSubsequenceAnimationElement&>(*AnimationElement));
			}

			if (!PayLoadKey.IsEmpty())
			{
				PayLoadsMap.Add(PayLoadKey, {FrameRate, AnimationElement});
			}
		}
	}

	void FLevelSequenceImportHelper::ProcessSubsequenceTrack(IDatasmithSubsequenceAnimationElement& SubsequenceAnimation)
	{
		TSharedPtr<IDatasmithLevelSequenceElement> PinnedTargetSequence = SubsequenceAnimation.GetSubsequence().Pin();
		if (!ensure(PinnedTargetSequence))
		{
			UE_LOGF(LogInterchangeDatasmith, Warning, "Cannot find LevelSequence element for Subsequence element %ls", SubsequenceAnimation.GetLabel());
			return;
		}

		UInterchangeAnimationTrackSetInstanceNode* TrackNode = NewObject< UInterchangeAnimationTrackSetInstanceNode >(&BaseNodeContainer);
		const FString TrackName = LevelSequenceNode->GetDisplayLabel() + TEXT("_SubTrack");
		const FString TrackNodeUid = NodeUtils::LevelSequencePrefix + TrackName;
		BaseNodeContainer.SetupNode(TrackNode, TrackNodeUid, TrackName, EInterchangeNodeContainerType::TranslatedScene);

		TrackNode->SetCustomStartFrame(SubsequenceAnimation.GetStartTime().Value);
		TrackNode->SetCustomDuration(SubsequenceAnimation.GetDuration());
		TrackNode->SetCustomTimeScale(SubsequenceAnimation.GetTimeScale());

		const FString SequenceNodeUid = NodeUtils::GetLevelSequenceUid(PinnedTargetSequence->GetName());
		TrackNode->SetCustomTrackSetDependencyUid(SequenceNodeUid);

		LevelSequenceNode->AddCustomAnimationTrackUid(TrackNodeUid);
	}

	FString FLevelSequenceImportHelper::ProcessVisibilityTrack(IDatasmithVisibilityAnimationElement& VisibilityAnimation, int32 TrackIndex)
	{
		UInterchangeAnimationTrackNode* TrackNode = AddTrackNode<UInterchangeAnimationTrackNode>(TrackIndex);
		if (!ensure(TrackNode))
		{
			UE_LOGF(LogInterchangeDatasmith, Warning, "Cannot create UInterchangeAnimationTrackNode object for Visibility element %ls", VisibilityAnimation.GetLabel());
			return {};
		}

		const FString ActorNodeUid = NodeUtils::GetActorUid(VisibilityAnimation.GetName());
		TrackNode->SetCustomActorDependencyUid(ActorNodeUid);

		TrackNode->SetCustomPropertyTrack(EInterchangePropertyTracks::ActorHiddenInGame);
		TrackNode->SetCustomAnimationPayloadKey(TrackNode->GetUniqueID(), EInterchangeAnimationPayLoadType::STEPCURVE);

		LevelSequenceNode->AddCustomAnimationTrackUid(TrackNode->GetUniqueID());

		return TrackNode->GetUniqueID();
	}

	FString FLevelSequenceImportHelper::ProcessTransformTrack(IDatasmithTransformAnimationElement& TransformAnimation, int32 TrackIndex)
	{
		UInterchangeTransformAnimationTrackNode* TrackNode = AddTrackNode< UInterchangeTransformAnimationTrackNode>(TrackIndex);
		if (!ensure(TrackNode))
		{
			UE_LOGF(LogInterchangeDatasmith, Warning, "Cannot create UInterchangeTransformAnimationTrackNode object for Visibility element %ls", TransformAnimation.GetLabel());
			return {};
		}

		const FString ActorNodeUid = NodeUtils::GetActorUid(TransformAnimation.GetName());
		TrackNode->SetCustomActorDependencyUid(ActorNodeUid);

		TrackNode->SetCustomCompletionMode((int32)TransformAnimation.GetCompletionMode());
		TrackNode->SetCustomUsedChannels((int32)TransformAnimation.GetEnabledTransformChannels());
		TrackNode->SetCustomAnimationPayloadKey(TrackNode->GetUniqueID(), EInterchangeAnimationPayLoadType::CURVE);

		LevelSequenceNode->AddCustomAnimationTrackUid(TrackNode->GetUniqueID());

		return TrackNode->GetUniqueID();
	}

	void FLevelSequenceImportHelper::AddLevelSequence(FLevelSequencePtr& LevelSequence, TArray<FLevelSequencePtr>& Sorted, TSet<FLevelSequencePtr>& Processed)
	{
		if (Processed.Contains(LevelSequence))
		{
			return;
		}

		const int32 NumAnimations = LevelSequence->GetAnimationsCount();
		for (int32 TrackIndex = 0; TrackIndex < NumAnimations; ++TrackIndex)
		{
			TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement = LevelSequence->GetAnimation(TrackIndex);
			if (!AnimationElement)
			{
				continue;
			}

			if (AnimationElement->IsSubType(EDatasmithElementAnimationSubType::SubsequenceAnimation))
			{
				IDatasmithSubsequenceAnimationElement& SubsequenceAnimation = static_cast<IDatasmithSubsequenceAnimationElement&>(*AnimationElement);
				TSharedPtr<IDatasmithLevelSequenceElement> PinnedTargetSequence = SubsequenceAnimation.GetSubsequence().Pin();
				if (PinnedTargetSequence && !Processed.Contains(PinnedTargetSequence))
				{
					AddLevelSequence(PinnedTargetSequence, Sorted, Processed);
				}
			}
		}

		Sorted.Add(LevelSequence);
		Processed.Add(LevelSequence);
	}

	void FLevelSequenceImportHelper::Sort(TArray<FLevelSequencePtr>& LevelSequences)
	{
		TSet<FLevelSequencePtr> Processed;
		TArray<FLevelSequencePtr> Sorted;
		Sorted.Reserve(LevelSequences.Num());

		for (TSharedPtr<IDatasmithLevelSequenceElement>& LevelSequence : LevelSequences)
		{
			AddLevelSequence(LevelSequence, Sorted, Processed);
		}

		LevelSequences.Reset();
		LevelSequences = MoveTemp(Sorted);
	}

	class FAnimationCurvesHelper
	{
	public:
		FAnimationCurvesHelper(const IDatasmithTransformAnimationElement& InAnimation, const FFrameRate& InSrcFrameRate, TArray<FRichCurve>& InCurves)
			: Animation(InAnimation)
			, SrcFrameRate(InSrcFrameRate)
			, Curves(InCurves)
		{
			ensure(Curves.Num() == 9);
		}

		bool FillCurves(EDatasmithTransformType TransformType)
		{
			bool bIsValidAnimation = false;
			const int32 NumFrames = Animation.GetFramesCount(TransformType);

			for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
			{
				if (Animation.GetFrame(TransformType, FrameIndex).IsValid())
				{
					bIsValidAnimation = true;
					break;
				}
			}

			if (!bIsValidAnimation)
			{
				return false;
			}

			static_assert((int)ERichCurveInterpMode::RCIM_Linear == (int)EDatasmithCurveInterpMode::Linear, "INVALID_ENUM_VALUE");
			static_assert((int)ERichCurveInterpMode::RCIM_Constant == (int)EDatasmithCurveInterpMode::Constant, "INVALID_ENUM_VALUE");
			static_assert((int)ERichCurveInterpMode::RCIM_Cubic == (int)EDatasmithCurveInterpMode::Cubic, "INVALID_ENUM_VALUE");

			const ERichCurveInterpMode Interpolation = (ERichCurveInterpMode)Animation.GetCurveInterpMode(TransformType);

			TFunction<FKeyHandle(FRichCurve&, float, float)> AddKey = [Interpolation](FRichCurve& Curve, float KeyTime, float KeyValue) -> FKeyHandle
			{
				FKeyHandle KeyHandle = Curve.AddKey(KeyTime, KeyValue);
				Curve.SetKeyInterpMode(KeyHandle, Interpolation);

				return KeyHandle;
			};

			const int32 CurveOffset = TransformType == EDatasmithTransformType::Translation ? 0 : (TransformType == EDatasmithTransformType::Rotation ? 3 : 6);

			FRichCurve& XCurve = Curves[CurveOffset + 0];
			FRichCurve& YCurve = Curves[CurveOffset + 1];
			FRichCurve& ZCurve = Curves[CurveOffset + 2];


			//  Rewind the rotations to prevent rotations that look like axis flips (eg. from 180 to -180 degrees)
			if (EDatasmithTransformType::Rotation == TransformType)
			{
				FKeyHandle XPrevKeyHandle;
				FKeyHandle YPrevKeyHandle;
				FKeyHandle ZPrevKeyHandle;
				int32 StartIndex = 1;

				for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
				{
					const FDatasmithTransformFrameInfo& FrameInfo = Animation.GetFrame(TransformType, FrameIndex);
					if (FrameInfo.IsValid())
					{
						const float FrameTime = SrcFrameRate.AsSeconds(FrameInfo.FrameNumber);

						XPrevKeyHandle = AddKey(XCurve, FrameTime, FrameInfo.X);
						YPrevKeyHandle = AddKey(YCurve, FrameTime, FrameInfo.Y);
						ZPrevKeyHandle = AddKey(ZCurve, FrameTime, FrameInfo.Z);

						break;
					}

					++StartIndex;
				}

				for (int32 FrameIndex = StartIndex; FrameIndex < NumFrames; ++FrameIndex)
				{
					const FDatasmithTransformFrameInfo& FrameInfo = Animation.GetFrame(TransformType, FrameIndex);
					if (FrameInfo.IsValid())
					{
						const float FrameTime = SrcFrameRate.AsSeconds(FrameInfo.FrameNumber);

						float KeyValue = FrameInfo.X;
						FMath::WindRelativeAnglesDegrees(XCurve.GetKeyValue(XPrevKeyHandle), KeyValue);
						XPrevKeyHandle = AddKey(XCurve, FrameTime, KeyValue);

						KeyValue = FrameInfo.Y;
						FMath::WindRelativeAnglesDegrees(YCurve.GetKeyValue(YPrevKeyHandle), KeyValue);
						YPrevKeyHandle = AddKey(YCurve, FrameTime, KeyValue);

						KeyValue = FrameInfo.Z;
						FMath::WindRelativeAnglesDegrees(ZCurve.GetKeyValue(ZPrevKeyHandle), KeyValue);
						ZPrevKeyHandle = AddKey(ZCurve, FrameTime, KeyValue);
					}
				}
			}
			else
			{
				for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
				{
					const FDatasmithTransformFrameInfo& FrameInfo = Animation.GetFrame(TransformType, FrameIndex);
					if (FrameInfo.IsValid())
					{
						const float FrameTime = SrcFrameRate.AsSeconds(FrameInfo.FrameNumber);

						AddKey(XCurve, FrameTime, FrameInfo.X);
						AddKey(YCurve, FrameTime, FrameInfo.Y);
						AddKey(ZCurve, FrameTime, FrameInfo.Z);
					}
				}
			}

			return true;
		}

	private:
		const IDatasmithTransformAnimationElement& Animation;
		const FFrameRate& SrcFrameRate;
		TArray<FRichCurve>& Curves;
	};

	bool GetAnimationPayloadData(const IDatasmithBaseAnimationElement& AnimationElement, float FrameRate, TArray<FRichCurve>& Curves)
	{
		using namespace UE::Interchange::Animation;

		if (AnimationElement.IsSubType(EDatasmithElementAnimationSubType::TransformAnimation))
		{
			const IDatasmithTransformAnimationElement& TransformAnimation = static_cast<const IDatasmithTransformAnimationElement&>(AnimationElement);

			Curves.SetNum(9);

			FAnimationCurvesHelper Helper(TransformAnimation, ConvertSampleRateToFrameRate(FrameRate), Curves);

			Helper.FillCurves(EDatasmithTransformType::Translation);
			Helper.FillCurves(EDatasmithTransformType::Rotation);
			Helper.FillCurves(EDatasmithTransformType::Scale);

			return true;
		}

		return false;
	}

	bool GetAnimationPayloadData(const IDatasmithBaseAnimationElement& AnimationElement, float FloatFrameRate, TArray<FInterchangeStepCurve>& StepCurves)
	{
		using namespace UE::Interchange::Animation;

		if (AnimationElement.IsSubType(EDatasmithElementAnimationSubType::VisibilityAnimation))
		{
			const IDatasmithVisibilityAnimationElement& VisibilityAnimation = static_cast<const IDatasmithVisibilityAnimationElement&>(AnimationElement);
			const FFrameRate FrameRate = ConvertSampleRateToFrameRate(FloatFrameRate);

			FInterchangeStepCurve& Curve = StepCurves.AddDefaulted_GetRef();

			const int32 NumFrames = VisibilityAnimation.GetFramesCount();

			TArray<float>& KeyTimes = Curve.KeyTimes;
			KeyTimes.Reserve(NumFrames);

			TArray<bool>& BooleanValues = Curve.BooleanKeyValues.Emplace();
			BooleanValues.Reserve(NumFrames);

			for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
			{
				const FDatasmithVisibilityFrameInfo& FrameInfo = VisibilityAnimation.GetFrame(FrameIndex);
				if (FrameInfo.IsValid())
				{
					KeyTimes.Add(FrameRate.AsSeconds(FrameInfo.FrameNumber));
					BooleanValues.Add(FrameInfo.bVisible);
				}
			}

			return true;
		}

		return false;
	}

	bool GetAnimationPayloadData(const IDatasmithBaseAnimationElement& AnimationElement, float FrameRate, EInterchangeAnimationPayLoadType PayLoadType, UE::Interchange::FAnimationPayloadData& PayLoadData)
	{
		switch (PayLoadType)
		{
		case EInterchangeAnimationPayLoadType::CURVE:
		case EInterchangeAnimationPayLoadType::MORPHTARGETCURVE:
			return GetAnimationPayloadData(AnimationElement, FrameRate, PayLoadData.Curves);

		case EInterchangeAnimationPayLoadType::STEPCURVE:
			return GetAnimationPayloadData(AnimationElement, FrameRate, PayLoadData.StepCurves);
		
		case EInterchangeAnimationPayLoadType::BAKED:
		case EInterchangeAnimationPayLoadType::NONE:
		default:
			break;
		}

		return false;
	}

	void TranslateLevelSequences(TArray<TSharedPtr<IDatasmithLevelSequenceElement>>& LevelSequences, UInterchangeBaseNodeContainer& BaseNodeContainer, TMap<FString, UE::DatasmithInterchange::AnimUtils::FAnimationPayloadDesc>& AnimationPayLoadMapping)
	{
		FLevelSequenceImportHelper LevelSequenceHelper(BaseNodeContainer);

		// Sort the array from less dependent to more dependent
		LevelSequenceHelper.Sort(LevelSequences);

		for (TSharedPtr<IDatasmithLevelSequenceElement>& SequenceElement : LevelSequences)
		{
			if (!SequenceElement || SequenceElement->GetAnimationsCount() == 0)
			{
				continue;
			}

			UInterchangeAnimationTrackSetNode* SequenceNode = NewObject< UInterchangeAnimationTrackSetNode >(&BaseNodeContainer);
			if (ensure(SequenceNode))
			{
				const FString SequenceName = SequenceElement->GetName();
				const FString SequenceNodeUid = NodeUtils::GetLevelSequenceUid(*SequenceName);
				BaseNodeContainer.SetupNode(SequenceNode, SequenceNodeUid, SequenceElement->GetLabel(), EInterchangeNodeContainerType::TranslatedScene);

				LevelSequenceHelper.ProcessLevelSequence(SequenceElement, SequenceNode);
			}
		}

		AnimationPayLoadMapping = MoveTemp(LevelSequenceHelper.PayLoadsMap);
	}
}

#undef LOCTEXT_NAMESPACE