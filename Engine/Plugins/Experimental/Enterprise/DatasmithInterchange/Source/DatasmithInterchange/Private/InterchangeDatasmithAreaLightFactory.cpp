// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithAreaLightFactory.h"

#include "Engine/Blueprint.h"
#include "InterchangeDatasmithLog.h"
#include "InterchangeDatasmithAreaLightFactoryNode.h"

#include "Scene/InterchangeActorHelper.h"
#include "DatasmithAreaLightActor.h"
#include "Engine/TextureLightProfile.h"
#include "InterchangeTextureFactoryNode.h"
#include "Logging/LogMacros.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"

UClass* UInterchangeDatasmithAreaLightFactory::GetFactoryClass() const
{
	FSoftObjectPath LightShapeBlueprintRef = FSoftObjectPath(TEXT("/DatasmithContent/Datasmith/DatasmithArealight.DatasmithArealight"));
	UBlueprint* LightShapeBlueprint = Cast< UBlueprint >(LightShapeBlueprintRef.ResolveObject());

	// LightShapeBlueprint should already be loaded at this point.
	//if (!ensure(LightShapeBlueprint))
	if (!LightShapeBlueprint)
	{
		return nullptr;
	}

	return LightShapeBlueprint->GeneratedClass;
}

UObject* UInterchangeDatasmithAreaLightFactory::ProcessActor(AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params)
{
	//using namespace UE::Interchange::ActorHelper;

	const UInterchangeDatasmithAreaLightFactoryNode* AreaLightFactoryNode = Cast<UInterchangeDatasmithAreaLightFactoryNode>(&FactoryNode);
	if (!AreaLightFactoryNode)
	{
		return nullptr;
	}

	ADatasmithAreaLightActor* AreaLightActor = Cast<ADatasmithAreaLightActor>(&SpawnedActor);
	if (!AreaLightActor)
	{
		return nullptr;
	}

	// Update AreaLight properties and rebuild the actor.
	{
		AreaLightActor->UnregisterAllComponents(true);

		// Find the IES Texture asset and apply it.
		FString IESTextureUid;
		if (AreaLightFactoryNode->GetCustomIESTexture(IESTextureUid))
		{
			if (const UInterchangeBaseNode* IESTextureNode = NodeContainer.GetNode(IESTextureUid))
			{
				TArray<FString> TargetNodesUid;
				IESTextureNode->GetTargetNodeUids(TargetNodesUid);
				for (const FString& TargetNodeUid : TargetNodesUid)
				{
					if (UInterchangeTextureFactoryNode* TextureFactoryNode = Cast<UInterchangeTextureFactoryNode>(NodeContainer.GetFactoryNode(TargetNodeUid)))
					{
						FSoftObjectPath TextureReferenceObject;
						TextureFactoryNode->GetCustomReferenceObject(TextureReferenceObject);
						if (UTextureLightProfile* TextureLightProfile = Cast<UTextureLightProfile>(TextureReferenceObject.TryLoad()))
						{
							AreaLightActor->IESTexture = TextureLightProfile;
							break;
						}
					}
				}
			}
		}

		// Apply all simple attributes.
		AreaLightFactoryNode->ApplyAllCustomAttributeToObject(AreaLightActor);

		AreaLightActor->RegisterAllComponents();
#if WITH_EDITOR
		AreaLightActor->RerunConstructionScripts();
#endif //WITH_EDITOR
	}

	return Super::ProcessActor(SpawnedActor, FactoryNode, NodeContainer, Params);
};