// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithAreaLightFactoryNode.h"

#include "DatasmithAreaLightActor.h"
#include "Engine/Blueprint.h"

UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomLightTypeKey = UE::Interchange::FAttributeKey(TEXT("LightType"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomLightShapeKey = UE::Interchange::FAttributeKey(TEXT("LightShape"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomDimensionsKey = UE::Interchange::FAttributeKey(TEXT("Dimensions"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomIntensityKey = UE::Interchange::FAttributeKey(TEXT("Intensity"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomIntensityUnitsKey = UE::Interchange::FAttributeKey(TEXT("IntensityUnits"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomColorKey = UE::Interchange::FAttributeKey(TEXT("Color"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomTemperatureKey = UE::Interchange::FAttributeKey(TEXT("Temperature"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomIESTextureKey = UE::Interchange::FAttributeKey(TEXT("IESTexture"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustombUseIESBrightnessKey = UE::Interchange::FAttributeKey(TEXT("UseIESBrightness"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomIESBrightnessScaleKey = UE::Interchange::FAttributeKey(TEXT("IESBrightnessScale"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomRotationKey = UE::Interchange::FAttributeKey(TEXT("Rotation"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomSourceRadiusKey = UE::Interchange::FAttributeKey(TEXT("SourceRadius"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomSourceLengthKey = UE::Interchange::FAttributeKey(TEXT("SourceLength"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomAttenuationRadiusKey = UE::Interchange::FAttributeKey(TEXT("AttenuationRadius"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomSpotlightInnerAngleKey = UE::Interchange::FAttributeKey(TEXT("SpotlightInnerAngle"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightFactoryNode::Macro_CustomSpotlightOuterAngleKey = UE::Interchange::FAttributeKey(TEXT("SpotlightOuterAngle"));

FString UInterchangeDatasmithAreaLightFactoryNode::GetTypeName() const
{
	const FString TypeName = TEXT("DatasmithAreaLightFactoryNode");
	return TypeName;
}

UClass* UInterchangeDatasmithAreaLightFactoryNode::GetObjectClass() const
{
	FSoftObjectPath LightShapeBlueprintRef = FSoftObjectPath(TEXT("/DatasmithContent/Datasmith/DatasmithArealight.DatasmithArealight"));
	UBlueprint* LightShapeBlueprint = Cast<UBlueprint>(LightShapeBlueprintRef.ResolveObject());

	// LightShapeBlueprint should be loaded at this point.
	if (!ensure(LightShapeBlueprint))
	{
		return nullptr;
	}

	return LightShapeBlueprint->GeneratedClass;
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomLightType(EDatasmithAreaLightActorType& OutAttributeValue) const
{
	uint8 AttributeValue = 0;
	bool bSuccess = [this, &AttributeValue]() { IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightType, uint8); }();
	OutAttributeValue = static_cast<EDatasmithAreaLightActorType>(AttributeValue);
	return bSuccess;
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomLightType(const EDatasmithAreaLightActorType& InAttributeValue, bool bAddApplyDelegate)
{
	uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, LightType, uint8, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomLightShape(EDatasmithAreaLightActorShape& OutAttributeValue) const
{
	uint8 AttributeValue = 0;
	bool bSuccess = [this, &AttributeValue]() { IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightShape, uint8); }();
	OutAttributeValue = static_cast<EDatasmithAreaLightActorShape>(AttributeValue);
	return bSuccess;
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomLightShape(const EDatasmithAreaLightActorShape& InAttributeValue, bool bAddApplyDelegate)
{
	uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, LightShape, uint8, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomDimensions(FVector2D& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Dimensions, FVector2D);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomDimensions(const FVector2D& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, Dimensions, FVector2D, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomIntensity(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Intensity, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomIntensity(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, Intensity, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomIntensityUnits(ELightUnits& OutAttributeValue) const
{
	uint8 AttributeValue = 0;
	bool bSuccess = [this, &AttributeValue]() { IMPLEMENT_NODE_ATTRIBUTE_GETTER(IntensityUnits, uint8); }();
	OutAttributeValue = static_cast<ELightUnits>(AttributeValue);
	return bSuccess;
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomIntensityUnits(const ELightUnits& InAttributeValue, bool bAddApplyDelegate)
{
	uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, IntensityUnits, uint8, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomColor(FLinearColor& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Color, FLinearColor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomColor(const FLinearColor& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, Color, FLinearColor, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomTemperature(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Temperature, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomTemperature(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, Temperature, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomIESTexture(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IESTexture, FString);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomIESTexture(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(IESTexture, FString);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomUseIESBrightness(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(bUseIESBrightness, bool);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomUseIESBrightness(const bool& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, bUseIESBrightness, bool, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomIESBrightnessScale(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(IESBrightnessScale, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomIESBrightnessScale(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, IESBrightnessScale, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomRotation(FRotator& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Rotation, FRotator);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomRotation(const FRotator& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, Rotation, FRotator, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomSourceRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceRadius, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomSourceRadius(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, SourceRadius, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomSourceLength(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceLength, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomSourceLength(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, SourceLength, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomAttenuationRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttenuationRadius, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomAttenuationRadius(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, AttenuationRadius, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomSpotlightInnerAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SpotlightInnerAngle, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomSpotlightInnerAngle(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, SpotlightInnerAngle, float, ADatasmithAreaLightActor);
}

bool UInterchangeDatasmithAreaLightFactoryNode::GetCustomSpotlightOuterAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SpotlightOuterAngle, float);
}

bool UInterchangeDatasmithAreaLightFactoryNode::SetCustomSpotlightOuterAngle(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeDatasmithAreaLightFactoryNode, SpotlightOuterAngle, float, ADatasmithAreaLightActor);
}

void UInterchangeDatasmithAreaLightFactoryNode::CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	Super::CopyWithObject(SourceNode, Object);

	if (const UInterchangeDatasmithAreaLightFactoryNode* ActorFactoryNode = Cast<UInterchangeDatasmithAreaLightFactoryNode>(SourceNode))
	{
		COPY_NODE_DELEGATES(ActorFactoryNode, LightType, EDatasmithAreaLightActorType, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, LightShape, EDatasmithAreaLightActorShape, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, Dimensions, FVector2D, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, Intensity, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, IntensityUnits, ELightUnits, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, Color, FLinearColor, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, Temperature, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, bUseIESBrightness, bool, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, IESBrightnessScale, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, Rotation, FRotator, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, SourceRadius, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, SourceLength, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, AttenuationRadius, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, SpotlightInnerAngle, float, ADatasmithAreaLightActor)
		COPY_NODE_DELEGATES(ActorFactoryNode, SpotlightOuterAngle, float, ADatasmithAreaLightActor)
	}
}
