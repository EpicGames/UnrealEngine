// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithAreaLightNode.h"



UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomLightTypeKey = UE::Interchange::FAttributeKey(TEXT("LightType"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomLightShapeKey = UE::Interchange::FAttributeKey(TEXT("LightShape"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomDimensionsKey = UE::Interchange::FAttributeKey(TEXT("Dimensions"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomColorKey = UE::Interchange::FAttributeKey(TEXT("Color"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomSourceRadiusKey = UE::Interchange::FAttributeKey(TEXT("SourceRadius"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomSourceLengthKey = UE::Interchange::FAttributeKey(TEXT("SourceLength"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomSpotlightInnerAngleKey = UE::Interchange::FAttributeKey(TEXT("SpotlightInnerAngle"));
UE::Interchange::FAttributeKey UInterchangeDatasmithAreaLightNode::Macro_CustomSpotlightOuterAngleKey = UE::Interchange::FAttributeKey(TEXT("SpotlightOuterAngle"));

bool UInterchangeDatasmithAreaLightNode::GetCustomLightType(EDatasmithAreaLightActorType& OutAttributeValue) const
{
	uint8 AttributeValue = 0;
	bool bSuccess = [this, &AttributeValue]() { IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightType, uint8); }();
	OutAttributeValue = static_cast<EDatasmithAreaLightActorType>(AttributeValue);
	return bSuccess;
}

bool UInterchangeDatasmithAreaLightNode::SetCustomLightType(const EDatasmithAreaLightActorType& InAttributeValue, bool bAddApplyDelegate)
{
	uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LightType, uint8);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomLightShape(EDatasmithAreaLightActorShape& OutAttributeValue) const
{
	uint8 AttributeValue = 0;
	bool bSuccess = [this, &AttributeValue]() { IMPLEMENT_NODE_ATTRIBUTE_GETTER(LightShape, uint8); }();
	OutAttributeValue = static_cast<EDatasmithAreaLightActorShape>(AttributeValue);
	return bSuccess;
}

bool UInterchangeDatasmithAreaLightNode::SetCustomLightShape(const EDatasmithAreaLightActorShape& InAttributeValue, bool bAddApplyDelegate)
{
	uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LightShape, uint8);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomDimensions(FVector2D& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Dimensions, FVector2D);
}

bool UInterchangeDatasmithAreaLightNode::SetCustomDimensions(const FVector2D& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Dimensions, FVector2D);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomColor(FLinearColor& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Color, FLinearColor);
}

bool UInterchangeDatasmithAreaLightNode::SetCustomColor(const FLinearColor& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(Color, FLinearColor);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomSourceRadius(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceRadius, float);
}

bool UInterchangeDatasmithAreaLightNode::SetCustomSourceRadius(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceRadius, float);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomSourceLength(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceLength, float);
}

bool UInterchangeDatasmithAreaLightNode::SetCustomSourceLength(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceLength, float);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomSpotlightInnerAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SpotlightInnerAngle, float);
}

bool UInterchangeDatasmithAreaLightNode::SetCustomSpotlightInnerAngle(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SpotlightInnerAngle, float);
}

bool UInterchangeDatasmithAreaLightNode::GetCustomSpotlightOuterAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SpotlightOuterAngle, float);
}

bool UInterchangeDatasmithAreaLightNode::SetCustomSpotlightOuterAngle(const float& AttributeValue, bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SpotlightOuterAngle, float);
}