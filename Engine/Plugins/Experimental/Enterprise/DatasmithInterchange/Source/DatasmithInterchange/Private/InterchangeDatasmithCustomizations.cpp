// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithCustomizations.h"

#if WITH_EDITOR
#include "InterchangeDatasmithTranslator.h"

#include "DetailLayoutBuilder.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IDetailCustomization> FInterchangeDatasmithTranslatorSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FInterchangeDatasmithTranslatorSettingsCustomization());
}

void FInterchangeDatasmithTranslatorSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);
	ensure(EditingObjects.Num() == 1);

	UInterchangeDatasmithTranslatorSettings* TranslatorSettings = Cast<UInterchangeDatasmithTranslatorSettings>(EditingObjects[0].Get());

	if (!ensure(TranslatorSettings && TranslatorSettings->DatasmithOption))
	{
		return;
	}

	TSharedRef<IPropertyHandle> DatasmithOptionHandle = DetailBuilder.GetProperty(TEXT("DatasmithOption"), UInterchangeDatasmithTranslatorSettings::StaticClass());

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	TSharedPtr<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	DetailsView->OnFinishedChangingProperties().AddLambda([DatasmithOptionHandle](const FPropertyChangedEvent& InPropertyChangedEvent)
		{
			DatasmithOptionHandle->NotifyFinishedChangingProperties();
		});

	DetailsView->SetObject(TranslatorSettings->DatasmithOption);

	IDetailPropertyRow* DetailPropertyRow = DetailBuilder.EditDefaultProperty(DatasmithOptionHandle);

	DetailPropertyRow->CustomWidget()
	[
		DetailsView.ToSharedRef()
	];
}

#endif