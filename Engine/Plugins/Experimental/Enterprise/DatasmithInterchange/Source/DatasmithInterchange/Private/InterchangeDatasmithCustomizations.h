// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;
class UInterchangeDatasmithTranslatorSettings;
struct FAssetData;

class FInterchangeDatasmithTranslatorSettingsCustomization : public IDetailCustomization
{
public:

	/**
	 * Creates an instance of this class
	 */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	/** End IDetailCustomization interface */
};
#endif
