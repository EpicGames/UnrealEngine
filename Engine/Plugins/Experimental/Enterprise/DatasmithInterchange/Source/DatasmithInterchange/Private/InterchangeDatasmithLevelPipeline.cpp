// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithLevelPipeline.h"

#include "InterchangeDatasmithAreaLightFactoryNode.h"
#include "InterchangeDatasmithAreaLightNode.h"
#include "InterchangeDatasmithUtils.h"

#include "InterchangeLevelSequenceFactoryNode.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMeshActorFactoryNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeStaticMeshFactoryNode.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Materials/MaterialInterface.h"
#include "Misc/PackageName.h"

void UInterchangeDatasmithLevelPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* NodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	using namespace UE::DatasmithInterchange;

	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

	// Add material factory dependencies for mesh actors where all overrides are filled with the same material
	for (UInterchangeMeshActorFactoryNode* MeshActorFactoryNode : NodeUtils::GetNodes<UInterchangeMeshActorFactoryNode>(NodeContainer))
	{
		// If applicable, transfer material override attribute from translated node to factory node
		TArray<FString> TargetNodes;
		MeshActorFactoryNode->GetTargetNodeUids(TargetNodes);
		if (TargetNodes.Num() == 0)
		{
			continue;
		}

		const UInterchangeSceneNode* SceneNode = Cast<UInterchangeSceneNode>(NodeContainer->GetNode(TargetNodes[0]));
		if (!SceneNode)
		{
			continue;
		}

		FString MaterialUid;
		if (!SceneNode->GetStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialUid))
		{
			continue;
		}

		const FString MaterialFactoryUid = UInterchangeMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(MaterialUid);
		MeshActorFactoryNode->AddFactoryDependencyUid(MaterialFactoryUid);
		MeshActorFactoryNode->AddStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialFactoryUid);
	}
}

void UInterchangeDatasmithLevelPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* NodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	using namespace UE::DatasmithInterchange;

	if (!NodeContainer || !CreatedAsset)
	{
		return;
	}

	Super::ExecutePostImportPipeline(NodeContainer, FactoryNodeKey, CreatedAsset, bIsAReimport);

	// If applicable, update material overrides for newly created mesh actor
	AStaticMeshActor* MeshActor = Cast<AStaticMeshActor>(CreatedAsset);
	UStaticMeshComponent* MeshComponent = MeshActor ? MeshActor->GetStaticMeshComponent() : nullptr;
	UStaticMesh* StaticMesh = MeshComponent ? MeshComponent->GetStaticMesh().Get() : nullptr;
	if (MeshActor && MeshComponent && StaticMesh)
	{
		const UInterchangeMeshActorFactoryNode* FactoryNode = Cast<UInterchangeMeshActorFactoryNode>(NodeContainer->GetFactoryNode(FactoryNodeKey));
		if (!FactoryNode)
		{
			return;
		}

		FString MaterialFactoryUid;
		if (!FactoryNode->GetStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialFactoryUid))
		{
			return;
		}

		const UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast< UInterchangeBaseMaterialFactoryNode>(NodeContainer->GetFactoryNode(MaterialFactoryUid));
		if (!MaterialFactoryNode)
		{
			return;
		}
		FSoftObjectPath MaterialFactoryNodeReferenceObject;
		MaterialFactoryNode->GetCustomReferenceObject(MaterialFactoryNodeReferenceObject);
		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialFactoryNodeReferenceObject.ResolveObject()))
		{
			const TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
			for (int32 Index = 0; Index < StaticMaterials.Num(); ++Index)
			{
				MeshComponent->SetMaterial(Index, MaterialInterface);
			}
		}
	}

	// If a ULevelSequence is referencing a ADatasmithAreaLightActor, update its mobility property
	if (const UInterchangeLevelSequenceFactoryNode* FactoryNode = Cast<UInterchangeLevelSequenceFactoryNode>(NodeContainer->GetFactoryNode(FactoryNodeKey)))
	{
		FSoftObjectPath FactoryNodeReferenceObject;
		FactoryNode->GetCustomReferenceObject(FactoryNodeReferenceObject);
		if (FactoryNodeReferenceObject.TryLoad() == CreatedAsset)
		{
			TArray<FString> AnimationTrackUids;
			FactoryNode->GetCustomAnimationTrackUids(AnimationTrackUids);

			for (const FString& AnimationTrackUid : AnimationTrackUids)
			{
				if (const UInterchangeBaseNode* TranslatedNode = NodeContainer->GetNode(AnimationTrackUid))
				{
					if (const UInterchangeTransformAnimationTrackNode* TransformTrackNode = Cast<UInterchangeTransformAnimationTrackNode>(TranslatedNode))
					{
						FString ActorNodeUid;
						if (TransformTrackNode->GetCustomActorDependencyUid(ActorNodeUid))
						{
							const FString ActorFactoryNodeUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ActorNodeUid);
							const UInterchangeFactoryBaseNode* ActorFactoryNode = Cast<UInterchangeFactoryBaseNode>(NodeContainer->GetNode(ActorFactoryNodeUid));

							if (ActorFactoryNode)
							{
								FSoftObjectPath ActorFactoryNodeReferenceObject;
								ActorFactoryNode->GetCustomReferenceObject(FactoryNodeReferenceObject);
								if (ADatasmithAreaLightActor* DatasmithAreaLightActor = Cast<ADatasmithAreaLightActor>(ActorFactoryNodeReferenceObject.TryLoad()))
								{
									DatasmithAreaLightActor->Mobility = EComponentMobility::Movable;
								}
							}
						}
					}
				}
			}
		}
	}
}

UInterchangeActorFactoryNode* UInterchangeDatasmithLevelPipeline::CreateActorFactoryNode(const UInterchangeBaseNode* TranslatedAssetNode) const
{
	if (!ensure(BaseNodeContainer))
	{
		return nullptr;
	}

	if (TranslatedAssetNode && TranslatedAssetNode->IsA<UInterchangeDatasmithAreaLightNode>())
	{
		return NewObject<UInterchangeDatasmithAreaLightFactoryNode>(BaseNodeContainer, NAME_None);
	}
	else
	{
		return Super::CreateActorFactoryNode(TranslatedAssetNode);
	}
}

void UInterchangeDatasmithLevelPipeline::SetUpFactoryNode(UInterchangeActorFactoryNode* ActorFactoryNode, const UInterchangeSceneNode* SceneNode,
	const UInterchangeBaseNode* TranslatedAssetNode, bool bIsRootJoint /* false */) const
{
	if (const UInterchangeDatasmithAreaLightNode* AreaLightNode = Cast<UInterchangeDatasmithAreaLightNode>(TranslatedAssetNode))
	{
		UInterchangeDatasmithAreaLightFactoryNode* AreaLightFactory = Cast<UInterchangeDatasmithAreaLightFactoryNode>(ActorFactoryNode);
		ensure(AreaLightFactory);
		SetupAreaLight(AreaLightFactory, AreaLightNode);
	}
	else
	{
		Super::SetUpFactoryNode(ActorFactoryNode, SceneNode, TranslatedAssetNode, bIsRootJoint);
	}
}

#define APPLY_FACTORY_ATTRIBUTE(AttributeName, AttributeType) \
	AttributeType AttributeName;\
	if (TranslatedNode->GetCustom##AttributeName(AttributeName))\
	{\
		FactoryNode->SetCustom##AttributeName(AttributeName);\
	}\

/**
 * Only modifies the attribute if a condition is satisfied
 * Condition should an expression evaluated to a boolean.
 */
#define APPLY_FACTORY_ATTRIBUTE_WITH_VALIDATION(AttributeName, AttributeType, Condition) \
	AttributeType AttributeName;\
	if (TranslatedNode->GetCustom##AttributeName(AttributeName))\
	{\
		if(Condition)\
		{\
			FactoryNode->SetCustom##AttributeName(AttributeName);\
		}\
	}\

/**
 * Allows Attributes to be modified before assignment.
 * A modifier could be a function, or a type conversion or anything else that can take the attribute value
 * as a parameter.
 */
#define APPLY_FACTORY_ATTRIBUTE_WITH_MODIFIER(AttributeName, AttributeType, Modifier) \
	AttributeType AttributeName;\
	if (TranslatedNode->GetCustom##AttributeName(AttributeName))\
	{\
		FactoryNode->SetCustom##AttributeName(Modifier(AttributeName));\
	}\

namespace UE::Interchange::AreaLightUtils
{
	// Helper function to correctly assign types that have been deprecated.
	EDatasmithAreaLightActorType GetLightActorType(const EDatasmithAreaLightActorType LightType)
	{
		EDatasmithAreaLightActorType LightActorType = EDatasmithAreaLightActorType::Point;

		switch (LightType)
		{
		case EDatasmithAreaLightActorType::Spot:
			LightActorType = EDatasmithAreaLightActorType::Spot;
			break;

		case EDatasmithAreaLightActorType::Point:
			LightActorType = EDatasmithAreaLightActorType::Point;
			break;

		case EDatasmithAreaLightActorType::IES_DEPRECATED:
			LightActorType = EDatasmithAreaLightActorType::Point;
			break;

		case EDatasmithAreaLightActorType::Rect:
			LightActorType = EDatasmithAreaLightActorType::Rect;
			break;
		}

		return LightActorType;
	}
}

void UInterchangeDatasmithLevelPipeline::SetupAreaLight(UInterchangeDatasmithAreaLightFactoryNode* FactoryNode, const UInterchangeDatasmithAreaLightNode* TranslatedNode) const
{	
	using namespace UE::Interchange::AreaLightUtils;

	APPLY_FACTORY_ATTRIBUTE_WITH_MODIFIER(LightType, EDatasmithAreaLightActorType, GetLightActorType);
	APPLY_FACTORY_ATTRIBUTE(LightShape, EDatasmithAreaLightActorShape);
	APPLY_FACTORY_ATTRIBUTE(Dimensions, FVector2D);
	
	APPLY_FACTORY_ATTRIBUTE(Intensity, float);
	EInterchangeLightUnits IntensityUnits;
	if(TranslatedNode->GetCustomIntensityUnits(IntensityUnits))
	{
		FactoryNode->SetCustomIntensityUnits(ELightUnits(IntensityUnits));
	}
	APPLY_FACTORY_ATTRIBUTE(Color, FLinearColor);
	APPLY_FACTORY_ATTRIBUTE(Temperature, float);

	APPLY_FACTORY_ATTRIBUTE(IESTexture, FString);
	APPLY_FACTORY_ATTRIBUTE(UseIESBrightness, bool);
	APPLY_FACTORY_ATTRIBUTE(IESBrightnessScale, float);
	APPLY_FACTORY_ATTRIBUTE(Rotation, FRotator);

	APPLY_FACTORY_ATTRIBUTE_WITH_VALIDATION(SourceRadius, float, SourceRadius > 0.0f);
	APPLY_FACTORY_ATTRIBUTE_WITH_VALIDATION(SourceLength, float, SourceLength > 0.0f);
	APPLY_FACTORY_ATTRIBUTE_WITH_VALIDATION(AttenuationRadius, float, AttenuationRadius > 0.0f);

	APPLY_FACTORY_ATTRIBUTE(SpotlightInnerAngle, float);
	APPLY_FACTORY_ATTRIBUTE(SpotlightOuterAngle, float);
}

#undef APPLY_FACTORY_ATTRIBUTE
#undef APPLY_FACTORY_ATTRIBUTE_WITH_VALIDATION
#undef APPLY_FACTORY_ATTRIBUTE_WITH_MODIFIER