// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeGenericScenesPipeline.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeDatasmithLevelPipeline.generated.h"

class UInterchangeBaseNodeContainer;
class UInterchangeSceneNode;
class UInterchangeDatasmithAreaLightFactoryNode;
class UInterchangeDatasmithAreaLightNode;
class UInterchangeDecalActorFactoryNode;
class UInterchangeDecalNode;

UCLASS(BlueprintType, Experimental)
class UInterchangeDatasmithLevelPipeline : public UInterchangeGenericLevelPipeline
{
	GENERATED_BODY()

protected:
	virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;
	virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual UInterchangeActorFactoryNode* CreateActorFactoryNode(const UInterchangeBaseNode* TranslatedAssetNode) const override;

	virtual void SetUpFactoryNode(UInterchangeActorFactoryNode* ActorFactoryNode, const UInterchangeSceneNode* SceneNode, const UInterchangeBaseNode* TranslatedAssetNode, bool bIsRootJoint = false) const override;

private:

	void SetupAreaLight(UInterchangeDatasmithAreaLightFactoryNode* AreaLightFactoryNode, const UInterchangeDatasmithAreaLightNode* AreaLightNode) const;

	friend class UInterchangeDatasmithPipeline;
};