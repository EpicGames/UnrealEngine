// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeDatasmithMaterialNode.h"
#include "InterchangeDatasmithPipeline.h"
#include "InterchangeDatasmithUtils.h"

#include "DatasmithMaterialElements.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"

#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "InterchangeManager.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeDecalMaterialNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "Async/TaskGraphInterfaces.h"
#include "Materials/MaterialExpression.h"
#include "Materials/MaterialExpressionCustom.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectRedirector.h"
#include "InterchangeMaterialInstanceNode.h"


const FString UInterchangeDatasmithPbrMaterialNode::ShadingModelAttrName(TEXT("Datasmith:UEPbrMaterial:ShadingModel"));
const FString UInterchangeDatasmithPbrMaterialNode::BlendModeAttrName(TEXT("Datasmith:UEPbrMaterial:BlendMode"));
const FString UInterchangeDatasmithPbrMaterialNode::OpacityMaskClipValueAttrName(TEXT("Datasmith:UEPbrMaterial:OpacityMaskClipValue"));
const FString UInterchangeDatasmithPbrMaterialNode::TranslucencyLightingModeAttrName(TEXT("Datasmith:UEPbrMaterial:TranslucencyLightingMode"));
const FString UInterchangeDatasmithPbrMaterialNode::MaterialFunctionsDependenciesKey(TEXT("Datasmith:UEPbrMaterial:MaterialFunctionsDependencies"));

namespace UE::DatasmithInterchange::MaterialUtils
{
	const FString MaterialFunctionPathAttrName(TEXT("Datasmith:Material:FunctionCall:MaterialFunctionPath"));
	const FString DefaultOutputIndexAttrName(TEXT("Datasmith:MaterialExpression:DefaultOutputIndex"));

	const FString MaterialTypeAttrName(TEXT("Datasmith:ReferenceMaterial:MaterialType"));
	const FString MaterialQualityAttrName(TEXT("Datasmith:ReferenceMaterial:MaterialQuality"));
	const FString MaterialParentAttrName(TEXT("Datasmith:ReferenceMaterial:MaterialParent"));

	class FPbrMaterialHelper
	{
	public:
		UInterchangeBaseNodeContainer& NodeContainer;
		FString MaterialID;
		IDatasmithUEPbrMaterialElement& MaterialElement;
		TMap<const IDatasmithMaterialExpression*, int32> ExpresionElements;
		TArray<UInterchangeShaderNode*> ProcessedExpressions;
		TSet<FString> ExpressionNames;
		TSet<FString> MaterialFunctionUids;

		struct FConnectionData
		{
			UInterchangeShaderNode* InputNode = nullptr;
			FString InputName;

			void ConnectDefaultOuputToInput(const FString& OutputNodeUid) const
			{
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(InputNode, InputName, OutputNodeUid);
			}

			void ConnectOuputToInput(const FString& OutputNodeUid, const FName& OutputName) const
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByName(InputNode, InputName, OutputNodeUid, OutputName.ToString());
			}

			void ConnectOuputToInput(const FString& OutputNodeUid, int32 OutputIndex) const
			{
				UInterchangeShaderPortsAPI::ConnectOuputToInputByIndex(InputNode, InputName, OutputNodeUid, OutputIndex);
			}
		};

		FPbrMaterialHelper(UInterchangeShaderNode& MaterialNode, IDatasmithUEPbrMaterialElement& InMaterialElement);

		void ConnectExpression(const IDatasmithMaterialExpressionTexture& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex);

		void ConnectExpression(const IDatasmithMaterialExpressionTextureCoordinate& DatasmithExpression, const FConnectionData& ConnectionData);

		void ConnectExpression(const IDatasmithMaterialExpressionFlattenNormal& DatasmithExpression, const FConnectionData& ConnectionData);

		void ConnectExpression(const IDatasmithMaterialExpressionGeneric& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex);

		void ConnectExpression(const IDatasmithMaterialExpressionFunctionCall& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex);

		void ConnectExpression(const IDatasmithMaterialExpressionCustom& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex);

		void ConnectExpression(const IDatasmithMaterialExpression* MaterialExpression, const FConnectionData& ConnectionData, int32 OutputIndex);

		static UClass* FindClass(const TCHAR* ClassName);

		static UClass* GetGenericExpressionClass(const TCHAR* GenericExpressionname);

		static UMaterialExpression* GetDefaultMaterialExpression(const TCHAR* GenericExpressionname);

		UInterchangeShaderNode* BuildGenericExpression(const IDatasmithMaterialExpressionGeneric& DatasmithExpression, const UInterchangeShaderNode& ParentNode, TArray<FName>* OutputNames = nullptr);

		template<class T = UInterchangeShaderNode>
		T* CreateExpressionNode(const IDatasmithMaterialExpression& DatasmithExpression, const FString& ParentNodeUid)
		{
			int32* ExpressionIndexPtr = ExpresionElements.Find(&DatasmithExpression);
			if (!ExpressionIndexPtr)
			{
				// TODO: Log error
				return nullptr;
			}

			if (ProcessedExpressions[*ExpressionIndexPtr] != nullptr)
			{
				return Cast<T>(ProcessedExpressions[*ExpressionIndexPtr]);
			}

			const FString NodeName = FDatasmithUtils::SanitizeObjectName(DatasmithExpression.GetName());
			const FString NodeUid = NodeUtils::MaterialExpressionPrefix + MaterialID + TEXT("_") + FString::FromInt(*ExpressionIndexPtr) + TEXT("_") + NodeName;
			
			UInterchangeShaderNode* ExpressionNode = NewObject< T >(&NodeContainer);
			NodeContainer.SetupNode(ExpressionNode, NodeUid, NodeName, EInterchangeNodeContainerType::TranslatedAsset, ParentNodeUid);

			ExpressionNode->AddInt32Attribute(DefaultOutputIndexAttrName, DatasmithExpression.GetDefaultOutputIndex());
			ProcessedExpressions[*ExpressionIndexPtr] = ExpressionNode;

			return Cast<T>(ExpressionNode);
		}

		EBlendMode GetUEPbrImportBlendMode();
	};

	bool BuildMaterialNode(IDatasmithMaterialInstanceElement& MaterialElement, UInterchangeMaterialInstanceNode& MaterialNode)
	{
		using namespace UE::Interchange::Materials;

		MaterialNode.AddInt32Attribute(MaterialTypeAttrName, int32(MaterialElement.GetMaterialType()));
		MaterialNode.AddInt32Attribute(MaterialQualityAttrName, int32(MaterialElement.GetQuality()));


		UInterchangeBaseNodeContainer* NodeContainer = Cast<UInterchangeBaseNodeContainer>(MaterialNode.GetOuter());
		check(NodeContainer);

		// Find matching reference material parameters
		for (int Index = 0; Index < MaterialElement.GetPropertiesCount(); ++Index)
		{
			const TSharedPtr< IDatasmithKeyValueProperty > Property = MaterialElement.GetProperty(Index);
			const FString InputValueKey = FDatasmithUtils::SanitizeObjectName(Property->GetName());

			switch (Property->GetPropertyType())
			{
				case EDatasmithKeyValuePropertyType::Color:
				{
					FLinearColor LinearColor;
					LinearColor.InitFromString(Property->GetValue());

					MaterialNode.AddVectorParameterValue(InputValueKey, LinearColor);
					break;
				}

				case EDatasmithKeyValuePropertyType::Float:
				{
					MaterialNode.AddScalarParameterValue(InputValueKey, FCString::Atof(Property->GetValue()));
					break;
				}

				case EDatasmithKeyValuePropertyType::Bool:
				{
					MaterialNode.AddStaticSwitchParameterValue(InputValueKey, FString(Property->GetValue()).ToBool());
					break;
				}

				case EDatasmithKeyValuePropertyType::Integer:
				{
					MaterialNode.AddInt32Attribute(UInterchangeShaderPortsAPI::MakeInputValueKey(InputValueKey), FCString::Atoi(Property->GetValue()));
					break;
				}

				case EDatasmithKeyValuePropertyType::Texture:
				{
					MaterialNode.AddTextureParameterValue(InputValueKey, Property->GetValue());
					break;
				}

				case EDatasmithKeyValuePropertyType::Vector:
				case EDatasmithKeyValuePropertyType::String:
				default:
				{
					break;
				}
			}
		}

		return true;
	}

	FPbrMaterialHelper::FPbrMaterialHelper(UInterchangeShaderNode& MaterialNode, IDatasmithUEPbrMaterialElement& InMaterialElement)
		: NodeContainer(*Cast<UInterchangeBaseNodeContainer>(MaterialNode.GetOuter()))
		, MaterialID(MaterialNode.GetUniqueID())
		, MaterialElement(InMaterialElement)
	{
		const int32 ExpressionsCount = MaterialElement.GetExpressionsCount();
		for (int32 ExpressionIndex = 0; ExpressionIndex < ExpressionsCount; ++ExpressionIndex)
		{
			if (const IDatasmithMaterialExpression* MaterialExpression = MaterialElement.GetExpression(ExpressionIndex))
			{
				ExpresionElements.Add(MaterialExpression, ExpressionIndex);
			}
		}

		ProcessedExpressions.AddZeroed(ExpressionsCount);
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpressionTexture& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes;

		if (OutputIndex < 0 || OutputIndex > 5)
		{
			// TODO: Log error
			return;
		}

		UInterchangeShaderNode* ExpressionNode = CreateExpressionNode(DatasmithExpression, ConnectionData.InputNode->GetUniqueID());
		if (!ensure(ExpressionNode))
		{
			// TODO: Log error
			return;
		}

		ExpressionNode->SetCustomShaderType(TextureSample::Name.ToString());
		const bool bIsAParameter = true;
		const FString TextureInputParameterName = TextureSample::Inputs::Texture.ToString();
		if (FPackageName::IsValidObjectPath(DatasmithExpression.GetTexturePathName()))
		{
			ExpressionNode->AddStringInput(TextureInputParameterName, DatasmithExpression.GetTexturePathName(), bIsAParameter);
		}
		else
		{
			const FString TextureUid = NodeUtils::TexturePrefix + FDatasmithUtils::SanitizeObjectName(DatasmithExpression.GetTexturePathName());
			if (NodeContainer.GetNode(TextureUid))
			{
				ExpressionNode->AddStringInput(TextureInputParameterName, TextureUid, bIsAParameter);
			}
			else
			{
				ensure(false);
				// TODO: Log warning
			}
		}

		static FName OutputNames[6] = { TextureSample::Outputs::RGB,
										TextureSample::Outputs::R,
										TextureSample::Outputs::G,
										TextureSample::Outputs::B,
										TextureSample::Outputs::A,
										TextureSample::Outputs::RGBA };


		ConnectionData.ConnectOuputToInput(ExpressionNode->GetUniqueID(), OutputNames[OutputIndex]);

		FConnectionData CoordConnectionData{ ExpressionNode, TextureSample::Inputs::Coordinates.ToString() };

		const IDatasmithExpressionInput& ExpressionInput = DatasmithExpression.GetInputCoordinate();
		ConnectExpression(ExpressionInput.GetExpression(), CoordConnectionData, ExpressionInput.GetOutputIndex());
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpressionTextureCoordinate& DatasmithExpression, const FConnectionData& ConnectionData)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes;

		UInterchangeShaderNode* ExpressionNode = CreateExpressionNode(DatasmithExpression, ConnectionData.InputNode->GetUniqueID());
		if (!ensure(ExpressionNode))
		{
			// TODO: Log error
			return;
		}

		ExpressionNode->SetCustomShaderType(TextureCoordinate::Name.ToString());

		{
			const FString ValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TextureCoordinate::Inputs::Index.ToString());
			ExpressionNode->AddInt32Attribute(ValueKey, DatasmithExpression.GetCoordinateIndex());
		}

		{
			const FString ValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TextureCoordinate::Inputs::UTiling.ToString());
			ExpressionNode->AddFloatAttribute(ValueKey, DatasmithExpression.GetUTiling());
		}

		{
			const FString ValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(TextureCoordinate::Inputs::VTiling.ToString());
			ExpressionNode->AddFloatAttribute(ValueKey, DatasmithExpression.GetVTiling());
		}

		ConnectionData.ConnectDefaultOuputToInput(ExpressionNode->GetUniqueID());
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpressionFlattenNormal& DatasmithExpression, const FConnectionData& ConnectionData)
	{
		using namespace UE::Interchange::Materials::Standard::Nodes;

		UInterchangeShaderNode* ExpressionNode = CreateExpressionNode(DatasmithExpression, ConnectionData.InputNode->GetUniqueID());
		if (!ensure(ExpressionNode))
		{
			// TODO: Log error
			return;
		}

		ExpressionNode->SetCustomShaderType(FlattenNormal::Name.ToString());

		{
			FConnectionData InputConnectionData{ ExpressionNode, FlattenNormal::Inputs::Normal.ToString() };

			const IDatasmithExpressionInput& ExpressionInput = DatasmithExpression.GetNormal();
			ConnectExpression(ExpressionInput.GetExpression(), InputConnectionData, ExpressionInput.GetOutputIndex());
		}

		{
			FConnectionData InputConnectionData{ ExpressionNode, FlattenNormal::Inputs::Flatness.ToString() };

			const IDatasmithExpressionInput& ExpressionInput = DatasmithExpression.GetFlatness();
			ConnectExpression(ExpressionInput.GetExpression(), InputConnectionData, ExpressionInput.GetOutputIndex());
		}

		ConnectionData.ConnectDefaultOuputToInput(ExpressionNode->GetUniqueID());
	}

	UClass* FPbrMaterialHelper::FindClass(const TCHAR* ClassName)
	{
		check(ClassName);

		if (UClass* Result = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Datasmith FindClass")))
		{
			return Result;
		}

		if (UObjectRedirector* RenamedClassRedirector = FindFirstObject<UObjectRedirector>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Datasmith FindClass")))
		{
			return CastChecked<UClass>(RenamedClassRedirector->DestinationObject);
		}

		return nullptr;
	}

	UClass* FPbrMaterialHelper::GetGenericExpressionClass(const TCHAR* GenericExpressionname)
	{
		const TCHAR* MaterialExpressionCharPtr = TEXT("MaterialExpression");
		const int32 MaterialExpressionLength = FCString::Strlen(MaterialExpressionCharPtr);
		const int32 ExpressionNameLength = FCString::Strlen(GenericExpressionname);

		FString ClassName;
		ClassName.Reserve(MaterialExpressionLength + ExpressionNameLength);
		ClassName.AppendChars(TEXT("MaterialExpression"), MaterialExpressionLength);
		ClassName.AppendChars(GenericExpressionname, ExpressionNameLength);
		return FindClass(*ClassName);
	}

	UMaterialExpression* FPbrMaterialHelper::GetDefaultMaterialExpression(const TCHAR* GenericExpressionname)
	{
		UClass* ExpressionClass = GetGenericExpressionClass(GenericExpressionname);

		if (!ensure(ExpressionClass))
		{
			return nullptr;
		}

		return ExpressionClass->GetDefaultObject<UMaterialExpression>();
	}

	UInterchangeShaderNode* FPbrMaterialHelper::BuildGenericExpression(const IDatasmithMaterialExpressionGeneric& DatasmithExpression, const UInterchangeShaderNode& ParentNode, TArray<FName>* OutputNames)
	{
		UInterchangeShaderNode* ExpressionNode = nullptr;

#if WITH_EDITOR
		UMaterialExpression* MaterialExpression = GetDefaultMaterialExpression(DatasmithExpression.GetExpressionName());

		if (!ensure(MaterialExpression))
		{
				// TODO: Log error
			return nullptr;
		}

		ExpressionNode = CreateExpressionNode(DatasmithExpression, ParentNode.GetUniqueID());
		if (!ensure(ExpressionNode))
		{
				// TODO: Log error
			return nullptr;
		}

		ExpressionNode->SetCustomShaderType(DatasmithExpression.GetExpressionName());

		// TODO: What about if MaterialExpression->HasAParameterName()?

		
		for (int32 PropertyIndex = 0; PropertyIndex < DatasmithExpression.GetPropertiesCount(); ++PropertyIndex)
		{
			const TSharedPtr<IDatasmithKeyValueProperty> Property = DatasmithExpression.GetProperty(PropertyIndex);

			switch (Property->GetPropertyType())
			{
			case EDatasmithKeyValuePropertyType::Float:
			{
				const FString PropertyKey = Property->GetName();
				const float PropertyValue = FCString::Atof(Property->GetValue());
				const bool bRequiresDelegate = true;
				UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_Float(ExpressionNode, PropertyKey, PropertyValue,FString(), bRequiresDelegate);
				break;
			}
			case EDatasmithKeyValuePropertyType::Texture:
			{
				using namespace UE::Interchange::Materials::Standard::Nodes;
				const FString TextureUid = NodeUtils::TexturePrefix + FDatasmithUtils::SanitizeObjectName(Property->GetValue());
				if (NodeContainer.GetNode(TextureUid))
				{
					// Textures are set in the MaterialFactory that require the TextureFactoryUid as opposed to TextureUid, 
					// hence assuming that the factory will be built for the texture, we are storing the TextureFactoryUid.
					const FString TextureFactoryUid = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TextureUid);

					// Passing it through the KeyValueProperty pipeline to ensure that it will be transferred to FactoryNode but no delegate would be added for this attribute.
					UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_FString(ExpressionNode, TextureSample::Inputs::Texture.ToString(), TextureFactoryUid, FString());
				}
				else
				{
					// Existing texture
					UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_FString(ExpressionNode, TextureSample::Inputs::Texture.ToString(), TextureUid, FString());
				}
				break;
			}
			// TODO: Can be added later.
			case EDatasmithKeyValuePropertyType::Bool:
			case EDatasmithKeyValuePropertyType::Vector:
			case EDatasmithKeyValuePropertyType::String:
			case EDatasmithKeyValuePropertyType::Color:
			case EDatasmithKeyValuePropertyType::Integer:
			default:
			{
				break;
			}
			}
		}
		

		for (int32 InputIndex = 0; InputIndex < DatasmithExpression.GetInputCount(); ++InputIndex)
		{
			if (const IDatasmithExpressionInput* ExpressionInput = DatasmithExpression.GetInput(InputIndex))
			{
				const FString InputName = MaterialExpression->GetInputName(InputIndex).ToString();
				FConnectionData InputConnectionData{ ExpressionNode, InputName };

				ConnectExpression(ExpressionInput->GetExpression(), InputConnectionData, ExpressionInput->GetOutputIndex());
			}
		}

		if (OutputNames != nullptr)
		{
			TArray<FExpressionOutput>& Outputs = MaterialExpression->GetOutputs();

			OutputNames->Reserve(Outputs.Num());

			for (FExpressionOutput& ExpressionOutput : Outputs)
			{
				OutputNames->Add(ExpressionOutput.OutputName);
			}
		}
#endif

		return ExpressionNode;
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpressionGeneric& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex)
	{
#if WITH_EDITOR
		TArray<FName> OutputNames;
		UInterchangeShaderNode* ExpressionNode = BuildGenericExpression(DatasmithExpression, *ConnectionData.InputNode, &OutputNames);
		if (!ensure(ExpressionNode))
		{
			// TODO: Log error
			return;
		}

		if (!ensure(OutputNames.IsValidIndex(OutputIndex)))
		{
			// TODO: Log error
			return;
		}

		UMaterialExpression* MaterialExpression = GetDefaultMaterialExpression(DatasmithExpression.GetExpressionName());

		// TODO: Handle case of Mask
		FExpressionOutput& ExpressionOutput = MaterialExpression->GetOutputs()[OutputIndex];
		ConnectionData.ConnectOuputToInput(ExpressionNode->GetUniqueID(), ExpressionOutput.OutputName);
#endif
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpressionFunctionCall& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex)
	{
#if WITH_EDITOR
		using namespace UE::Interchange::Materials::Standard::Nodes;

		FString MaterialUid;
		FString FunctionPath = FPackageName::ExportTextPathToObjectPath(DatasmithExpression.GetFunctionPathName());

		TArray<FString> InputNames;

		if (FPackageName::DoesPackageExist(FunctionPath))
		{
			FSoftObjectPath ObjectPath(TEXT("MaterialFunction'") + FunctionPath + TEXT("'"));

			UMaterialFunctionInterface* MaterialFunction = Cast<UMaterialFunctionInterface>(ObjectPath.TryLoad());

			// TODO: The call to TryLoad may fail if the asset was not loaded ahead of time.
			// Any asset starting with /Engine may work
			if (!ensure(MaterialFunction))
			{
				//TODO: Log warning
				return;
			}

			TArray<FFunctionExpressionInput> FunctionInputs;
			TArray<FFunctionExpressionOutput> FunctionOutputs;

			MaterialFunction->UpdateFromFunctionResource();
			MaterialFunction->GetInputsAndOutputs(FunctionInputs, FunctionOutputs);

			const bool bCanProceed = FunctionInputs.Num() < DatasmithExpression.GetInputCount() || FunctionOutputs.IsValidIndex(OutputIndex);
			if (!ensure(bCanProceed))
			{
				// TODO: Log warning invalid predefined asset reference used by function call expression
				return;
			}

			InputNames.Reserve(FunctionInputs.Num());
			for (FFunctionExpressionInput& FunctionInput : FunctionInputs)
			{
				InputNames.Add(FunctionInput.Input.InputName.ToString());
			}
		}
		else
		{
			MaterialUid = NodeUtils::MaterialFunctionPrefix + FunctionPath;
			const UInterchangeDatasmithPbrMaterialNode* PbrMaterialNode = Cast<const UInterchangeDatasmithPbrMaterialNode>(NodeContainer.GetNode(MaterialUid));
			if (!ensure(PbrMaterialNode))
			{
				// TODO: Log warning invalid internal reference used by function call expression
				return;
			}

			FunctionPath.Empty();
			UInterchangeShaderPortsAPI::GatherInputs(PbrMaterialNode, InputNames);

			MaterialFunctionUids.Add(MaterialUid);
		}

		if (FunctionPath.IsEmpty() && MaterialUid.IsEmpty())
		{
			// TODO: Log warning invalid reference used by function call expression
			return;
		}

		UInterchangeFunctionCallShaderNode* FunctionCallExpressionNode = CreateExpressionNode<UInterchangeFunctionCallShaderNode>(DatasmithExpression, ConnectionData.InputNode->GetUniqueID());
		if (!ensure(FunctionCallExpressionNode))
		{
			// TODO: Log error
			return;
		}

		if (!MaterialUid.IsEmpty())
		{
			FunctionCallExpressionNode->SetCustomMaterialFunction(MaterialUid);
		}
		else
		{
			FunctionCallExpressionNode->SetCustomMaterialFunction(FunctionPath);
		}

		ConnectionData.ConnectOuputToInput(FunctionCallExpressionNode->GetUniqueID(), OutputIndex);

		for (int32 InputIndex = 0; InputIndex < DatasmithExpression.GetInputCount(); ++InputIndex)
		{
			if (const IDatasmithExpressionInput* ExpressionInput = DatasmithExpression.GetInput(InputIndex))
			{
				FConnectionData InputConnectionData{ FunctionCallExpressionNode, InputNames[InputIndex] };

				ConnectExpression(ExpressionInput->GetExpression(), InputConnectionData, ExpressionInput->GetOutputIndex());
			}
		}
#endif
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpressionCustom& DatasmithExpression, const FConnectionData& ConnectionData, int32 OutputIndex)
	{
		// TODO: See FDatasmithMaterialExpressions::CreateCustomExpression
	}

	void FPbrMaterialHelper::ConnectExpression(const IDatasmithMaterialExpression* MaterialExpression, const FConnectionData& ConnectionData, int32 OutputIndex)
	{
		if (!MaterialExpression)
		{
			return;
		}

		int32* ExpressionIndexPtr = ExpresionElements.Find(MaterialExpression);
		if (ExpressionIndexPtr && ProcessedExpressions[*ExpressionIndexPtr])
		{
			UInterchangeShaderNode* ExpressionNode = ProcessedExpressions[*ExpressionIndexPtr];

			int32 DefaultOutputIndex = 0;
			ExpressionNode->GetInt32Attribute(DefaultOutputIndexAttrName, DefaultOutputIndex);

			if (OutputIndex == DefaultOutputIndex)
			{
					ConnectionData.ConnectDefaultOuputToInput(ExpressionNode->GetUniqueID());
			}
			else
			{
				ConnectionData.ConnectOuputToInput(ExpressionNode->GetUniqueID(), OutputIndex);
			}
		}

		const FString ParameterName(MaterialExpression->GetName());
		const bool bIsAParameter = !ParameterName.IsEmpty();

		if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::ConstantBool))
		{
			using namespace UE::Interchange::Materials::Standard::Nodes;

			const IDatasmithMaterialExpressionBool* BoolExpression = static_cast<const IDatasmithMaterialExpressionBool*>(MaterialExpression);
			
			ensure(!ParameterName.IsEmpty());

			UInterchangeShaderNode* ExpressionNode = CreateExpressionNode(*BoolExpression, ConnectionData.InputNode->GetUniqueID());
			if (!ensure(ExpressionNode))
			{
				// TODO: Log error
				return;
			}

			const FString InputName(UInterchangeShaderPortsAPI::MakeInputParameterKey(StaticBoolParameter::Attributes::DefaultValue.ToString()));
			ExpressionNode->AddBooleanAttribute(InputName, BoolExpression->GetBool());
			ExpressionNode->SetCustomShaderType(StaticBoolParameter::Name.ToString());
			UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ConnectionData.InputNode, ConnectionData.InputName, ExpressionNode->GetUniqueID());
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::ConstantScalar))
		{
			using namespace UE::Interchange::Materials::Standard::Nodes;

			const IDatasmithMaterialExpressionScalar* ScalarExpression = static_cast<const IDatasmithMaterialExpressionScalar*>(MaterialExpression);

			if (bIsAParameter)
			{
				UInterchangeShaderNode* ExpressionNode = CreateExpressionNode(*ScalarExpression, ConnectionData.InputNode->GetUniqueID());
				if (!ensure(ExpressionNode))
				{
					// TODO: Log error
					return;
				}

				ExpressionNode->AddFloatInput(ScalarParameter::Attributes::DefaultValue.ToString(), ScalarExpression->GetScalar(), bIsAParameter);
				ExpressionNode->SetCustomShaderType(ScalarParameter::Name.ToString());
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ConnectionData.InputNode, ConnectionData.InputName, ExpressionNode->GetUniqueID());
			}
			else
			{
				ConnectionData.InputNode->AddFloatInput(ConnectionData.InputName, ScalarExpression->GetScalar(), bIsAParameter);
			}
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::ConstantColor))
		{
			using namespace UE::Interchange::Materials::Standard::Nodes;

			const IDatasmithMaterialExpressionColor* ColorExpression = static_cast<const IDatasmithMaterialExpressionColor*>(MaterialExpression);

			if (bIsAParameter)
			{
				UInterchangeShaderNode* ExpressionNode = CreateExpressionNode(*ColorExpression, ConnectionData.InputNode->GetUniqueID());
				if (!ensure(ExpressionNode))
				{
					// TODO: Log error
					return;
				}

				ExpressionNode->AddLinearColorInput(VectorParameter::Attributes::DefaultValue.ToString(), ColorExpression->GetColor(), bIsAParameter);
				ExpressionNode->SetCustomShaderType(VectorParameter::Name.ToString());
				UInterchangeShaderPortsAPI::ConnectDefaultOuputToInput(ConnectionData.InputNode, ConnectionData.InputName, ExpressionNode->GetUniqueID());
			}
			else
			{
				ConnectionData.InputNode->AddLinearColorInput(ConnectionData.InputName, ColorExpression->GetColor(), bIsAParameter);
			}
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::Texture))
		{
			ConnectExpression(static_cast<const IDatasmithMaterialExpressionTexture&>(*MaterialExpression), ConnectionData, OutputIndex);
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::TextureCoordinate))
		{
			ConnectExpression(static_cast<const IDatasmithMaterialExpressionTextureCoordinate&>(*MaterialExpression), ConnectionData);
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::FlattenNormal))
		{
			ConnectExpression(static_cast<const IDatasmithMaterialExpressionFlattenNormal&>(*MaterialExpression), ConnectionData);
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::Generic))
		{
			ConnectExpression(static_cast<const IDatasmithMaterialExpressionGeneric&>(*MaterialExpression), ConnectionData, OutputIndex);
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
		{
			ConnectExpression(static_cast<const IDatasmithMaterialExpressionFunctionCall&>(*MaterialExpression), ConnectionData, OutputIndex);
		}
		else if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::Custom))
		{
			ConnectExpression(static_cast<const IDatasmithMaterialExpressionCustom&>(*MaterialExpression), ConnectionData, OutputIndex);
		}
	}

	EBlendMode FPbrMaterialHelper::GetUEPbrImportBlendMode()
	{
		EBlendMode BlendMode = static_cast<EBlendMode>(MaterialElement.GetBlendMode());

		if (MaterialElement.GetOpacity().GetExpression() && BlendMode != BLEND_Translucent && BlendMode != BLEND_Masked)
		{
			return EBlendMode::BLEND_Translucent; // force translucent
		}

		// If this material is dependent on function call expression, we need to check if those are translucent to apply
		// translucency attribute to this material too.
		for (int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement.GetExpressionsCount(); ExpressionIndex++)
		{
			IDatasmithMaterialExpression* MaterialExpression = MaterialElement.GetExpression(ExpressionIndex);
			if (MaterialExpression && MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
			{
				IDatasmithMaterialExpressionFunctionCall* FunctionCall = static_cast<IDatasmithMaterialExpressionFunctionCall*>(MaterialExpression);

				FString MaterialFunctionNodeUid = NodeUtils::MaterialFunctionPrefix + FunctionCall->GetFunctionPathName();
				if (const UInterchangeDatasmithPbrMaterialNode* MaterialFunctionNode = Cast<const UInterchangeDatasmithPbrMaterialNode>(NodeContainer.GetNode(MaterialFunctionNodeUid)))
				{
					EBlendMode MaterialFunctionBlendMode;
					if (MaterialFunctionNode->GetCustomBlendMode(MaterialFunctionBlendMode) && MaterialFunctionBlendMode == EBlendMode::BLEND_Translucent)
					{
						return MaterialFunctionBlendMode;
					}
				}
			}
		}

		return BlendMode;
	}

	bool BuildMaterialNode(IDatasmithUEPbrMaterialElement& MaterialElement, UInterchangeDatasmithPbrMaterialNode& MaterialNode)
	{
		using namespace UE::Interchange::Materials;

		FPbrMaterialHelper NodeHelper(MaterialNode, MaterialElement);

		TFunction<void(IDatasmithExpressionInput&, FName)> ConnectOutputToInput;
		ConnectOutputToInput = [&NodeHelper, &MaterialNode](const IDatasmithExpressionInput& ExpressionInput, FName InputName) -> void
		{
			FPbrMaterialHelper::FConnectionData ConnectionData{ &MaterialNode , InputName.ToString()};
			NodeHelper.ConnectExpression(ExpressionInput.GetExpression(), ConnectionData, ExpressionInput.GetOutputIndex());
		};

		if (MaterialElement.GetUseMaterialAttributes())
		{
			//We ignore all the other inputs if we are using MaterialAttributes.
			ConnectOutputToInput(MaterialElement.GetMaterialAttributes(), Common::Parameters::BxDF);
		}
		else
		{
			ConnectOutputToInput(MaterialElement.GetBaseColor(), PBRMR::Parameters::BaseColor);

			ConnectOutputToInput(MaterialElement.GetMetallic(), PBRMR::Parameters::Metallic);

			ConnectOutputToInput(MaterialElement.GetSpecular(), PBRMR::Parameters::Specular);

			ConnectOutputToInput(MaterialElement.GetRoughness(), PBRMR::Parameters::Roughness);

			ConnectOutputToInput(MaterialElement.GetEmissiveColor(), PBRMR::Parameters::EmissiveColor);

			ConnectOutputToInput(MaterialElement.GetAmbientOcclusion(), PBRMR::Parameters::Occlusion);

			ConnectOutputToInput(MaterialElement.GetNormal(), PBRMR::Parameters::Normal);

			ConnectOutputToInput(MaterialElement.GetOpacity(), PBRMR::Parameters::Opacity);

			ConnectOutputToInput(MaterialElement.GetRefraction(), PBRMR::Parameters::IndexOfRefraction);

			ConnectOutputToInput(MaterialElement.GetClearCoat(), ClearCoat::Parameters::ClearCoat);

			ConnectOutputToInput(MaterialElement.GetClearCoatRoughness(), ClearCoat::Parameters::ClearCoatRoughness);
		}

		for (const FString& MaterialFunctionUid : NodeHelper.MaterialFunctionUids)
		{
			MaterialNode.AddMaterialFunctionsDependency(MaterialFunctionUid);
		}

		MaterialNode.SetCustomTwoSided(MaterialElement.GetTwoSided());
		MaterialNode.SetCustomOpacityMaskClipValue(MaterialElement.GetOpacityMaskClipValue());
		MaterialNode.SetCustomTranslucencyLightingMode(static_cast<ETranslucencyLightingMode>(MaterialElement.GetTranslucencyLightingMode()));
		MaterialNode.SetCustomBlendMode(NodeHelper.GetUEPbrImportBlendMode());

		switch (MaterialElement.GetShadingModel())
		{
		case EDatasmithShadingModel::Subsurface:
			MaterialNode.SetCustomShadingModel(MSM_Subsurface);
			break;

		case EDatasmithShadingModel::ClearCoat:
			MaterialNode.SetCustomShadingModel(MSM_ClearCoat);
			break;

		case EDatasmithShadingModel::ThinTranslucent:
			MaterialNode.SetCustomShadingModel(MSM_ThinTranslucent);
			if (MaterialElement.GetOpacity().GetExpression())
			{
				MaterialNode.SetCustomTranslucencyLightingMode(ETranslucencyLightingMode::TLM_SurfacePerPixelLighting);
			}
			break;
		case EDatasmithShadingModel::Unlit:
			MaterialNode.SetCustomShadingModel(MSM_Unlit);
			break;
		default:
			break;
		}

		if (MaterialElement.GetMaterialFunctionOnly())
		{
			MaterialNode.SetCustomIsAShaderFunction(true);
		}

		// Connect expressions to any UMaterialExpressionCustomOutput since these aren't part of the predefined material outputs
		for (int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement.GetExpressionsCount(); ++ExpressionIndex)
		{
			if (!NodeHelper.ProcessedExpressions[ExpressionIndex])
			{
				if (IDatasmithMaterialExpression* MaterialExpression = MaterialElement.GetExpression(ExpressionIndex))
				{
					// Unconnected generic expressions usually are custom output, connect them to the appropriate channel
					if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::Generic))
					{
						IDatasmithMaterialExpressionGeneric& GenericExpression = static_cast<IDatasmithMaterialExpressionGeneric&>(*MaterialExpression);
						if (!FCString::Strcmp(GenericExpression.GetExpressionName(), TEXT("ThinTranslucentMaterialOutput")))
						{
							ensure(GenericExpression.GetInputCount() == 1);
							ConnectOutputToInput(*GenericExpression.GetInput(0), ThinTranslucent::Parameters::TransmissionColor);
						}
						else if (!FCString::Strcmp(GenericExpression.GetExpressionName(), TEXT("ClearCoatNormalCustomOutput")))
						{
							ensure(GenericExpression.GetInputCount() == 1);
							ConnectOutputToInput(*GenericExpression.GetInput(0), ClearCoat::Parameters::ClearCoatNormal);
						}
					}
					else
					{
						ensure(MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::ConstantBool) ||
							MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::ConstantColor) ||
							MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::ConstantScalar));
					}
				}
			}
		}

		return true;
	}

	bool BuildMaterialNode(IDatasmithDecalMaterialElement& MaterialElement, UInterchangeDecalMaterialNode& MaterialNode)
	{
		// #todo_Vedang_Javdekar Check if adding this attribute would help later in the Import Process.
		//MaterialNode.AddInt32Attribute(MaterialTypeAttrName, int32(EDatasmithReferenceMaterialType::Decal));

		MaterialNode.SetCustomDiffuseTexturePath(MaterialElement.GetDiffuseTexturePathName());
		MaterialNode.SetCustomNormalTexturePath(MaterialElement.GetNormalTexturePathName());

		return true;
	}

	void ProcessDependencies(IDatasmithUEPbrMaterialElement& MaterialElement, TArray<TSharedPtr<IDatasmithBaseMaterialElement>>& OutMaterialElements, TMap<FString, TSharedPtr<IDatasmithBaseMaterialElement>>& MaterialsToSort)
	{
		for (int32 ExpressionIndex = 0; ExpressionIndex < MaterialElement.GetExpressionsCount(); ExpressionIndex++)
		{
			IDatasmithMaterialExpression* MaterialExpression = MaterialElement.GetExpression(ExpressionIndex);
			if (MaterialExpression && MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
			{
				IDatasmithMaterialExpressionFunctionCall* MaterialExpressionFunctionCall = static_cast<IDatasmithMaterialExpressionFunctionCall*>(MaterialExpression);

				TSharedPtr<IDatasmithBaseMaterialElement> MaterialFunction;
				if (MaterialsToSort.RemoveAndCopyValue(MaterialExpressionFunctionCall->GetFunctionPathName(), MaterialFunction))
				{
					ProcessDependencies(static_cast<IDatasmithUEPbrMaterialElement&>(*MaterialFunction), OutMaterialElements, MaterialsToSort);

					OutMaterialElements.Add(MaterialFunction);
				}
			}
		}
	}

	// Preload in memory all the material functions referenced in the Pbr materials
	// Rationale: The translator is not done in the game thread and some material function might not have been
	//			  loaded yet. They need to be loaded during the translation in order to get the name of their inputs.
	void PreLoadReferencedAssets(const TArray<TSharedPtr<IDatasmithBaseMaterialElement>>& MaterialElements)
	{
#if WITH_EDITOR
		if (IsInGameThread())
		{
			return;
		}

		TSet<FString> ProcessedMaterialFunctions;
		FGraphEventArray ThingsToComplete;
		ThingsToComplete.Reserve(MaterialElements.Num());

		for (const TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement : MaterialElements)
		{
			if (MaterialElement && MaterialElement->IsA(EDatasmithElementType::UEPbrMaterial))
			{
				IDatasmithUEPbrMaterialElement& PbrMaterialElement = static_cast<IDatasmithUEPbrMaterialElement&>(*MaterialElement);

				for (int32 ExpressionIndex = 0; ExpressionIndex < PbrMaterialElement.GetExpressionsCount(); ExpressionIndex++)
				{
					IDatasmithMaterialExpression* MaterialExpression = PbrMaterialElement.GetExpression(ExpressionIndex);
					if (MaterialExpression && MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
					{
						IDatasmithMaterialExpressionFunctionCall* FunctionCall = static_cast<IDatasmithMaterialExpressionFunctionCall*>(MaterialExpression);

						FString FunctionPath = FPackageName::ExportTextPathToObjectPath(FunctionCall->GetFunctionPathName());

						if (!ProcessedMaterialFunctions.Contains(FunctionPath) && FPackageName::DoesPackageExist(FunctionPath))
						{
							ProcessedMaterialFunctions.Add(FunctionPath);

							FSoftObjectPath ObjectPath(TEXT("MaterialFunction'") + FunctionPath + TEXT("'"));
							if (!ObjectPath.ResolveObject())
							{
								TFunction<void()> Task = [FunctionPath]() -> void
								{
									FSoftObjectPath ObjectPath(TEXT("MaterialFunction'") + FunctionPath + TEXT("'"));
									ensure(ObjectPath.TryLoad());
								};

								ThingsToComplete.Add(FFunctionGraphTask::CreateAndDispatchWhenReady(MoveTemp(Task), TStatId(), nullptr, ENamedThreads::GameThread));
							}
						}
					}
				}
			}
		}

		if (ThingsToComplete.Num())
		{
			FTaskGraphInterface::Get().WaitUntilTasksComplete(ThingsToComplete, ENamedThreads::AnyThread);
		}
#endif
	}

	void ProcessMaterialElements(TArray<TSharedPtr<IDatasmithBaseMaterialElement>>& InOutMaterialElements)
	{
		PreLoadReferencedAssets(InOutMaterialElements);

		bool bNeedSorting = false;

		for (TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement : InOutMaterialElements)
		{
			if (MaterialElement->IsA(EDatasmithElementType::UEPbrMaterial))
			{
				bNeedSorting = true;
				break;
			}
		}

		if (!bNeedSorting)
		{
			return;
		}

		TMap<FString, TSharedPtr<IDatasmithBaseMaterialElement>> MaterialsToSort;
		TArray<TSharedPtr<IDatasmithBaseMaterialElement>> MaterialElements(InOutMaterialElements);

		MaterialsToSort.Reserve(MaterialElements.Num());
		InOutMaterialElements.Empty(MaterialElements.Num());

		for (TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement : MaterialElements)
		{
			if (MaterialElement->IsA(EDatasmithElementType::UEPbrMaterial))
			{
				MaterialsToSort.Add(MaterialElement->GetName(), MaterialElement);
				continue;
			}

			InOutMaterialElements.Add(MaterialElement);
		}

		TArray<FString> MaterialNames;
		MaterialsToSort.GenerateKeyArray(MaterialNames);

		for (const FString& MaterialName : MaterialNames)
		{
			TSharedPtr<IDatasmithBaseMaterialElement> MaterialFunction;
			if (MaterialsToSort.RemoveAndCopyValue(MaterialName, MaterialFunction))
			{
				ProcessDependencies(static_cast<IDatasmithUEPbrMaterialElement&>(*MaterialFunction), InOutMaterialElements, MaterialsToSort);

				InOutMaterialElements.Add(MaterialFunction);
			}
		}
	}

	UInterchangeBaseNode* AddMaterialNode(const TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, UInterchangeBaseNodeContainer& NodeContainer)
	{
		UInterchangeBaseNode* BaseNode = nullptr;

		if (MaterialElement.IsValid())
		{
			if (MaterialElement->IsA(EDatasmithElementType::MaterialInstance))
			{
				UInterchangeMaterialInstanceNode* MaterialNode = NewObject<UInterchangeMaterialInstanceNode>(&NodeContainer);
				const FString MaterialNodeUid = NodeUtils::MaterialPrefix + FDatasmithUtils::SanitizeObjectName(MaterialElement->GetName());
				NodeContainer.SetupNode(MaterialNode, MaterialNodeUid, *FDatasmithUtils::SanitizeObjectName(MaterialElement->GetLabel()), EInterchangeNodeContainerType::TranslatedAsset);

				if (!BuildMaterialNode(static_cast<IDatasmithMaterialInstanceElement&>(*MaterialElement), *MaterialNode))
				{
					MaterialNode = nullptr;
				}

				BaseNode = MaterialNode;
			}
			else if (MaterialElement->IsA(EDatasmithElementType::UEPbrMaterial))
			{
				IDatasmithUEPbrMaterialElement& PbrMaterialElement = static_cast<IDatasmithUEPbrMaterialElement&>(*MaterialElement);

				UInterchangeDatasmithPbrMaterialNode* MaterialNode = NewObject<UInterchangeDatasmithPbrMaterialNode>(&NodeContainer);

				const FString& Prefix = PbrMaterialElement.GetMaterialFunctionOnly() ? NodeUtils::MaterialFunctionPrefix : NodeUtils::MaterialPrefix;
				const FString MaterialNodeUid = Prefix + FDatasmithUtils::SanitizeObjectName(MaterialElement->GetName());
				NodeContainer.SetupNode(MaterialNode, MaterialNodeUid, MaterialElement->GetLabel(), EInterchangeNodeContainerType::TranslatedAsset);

				if (!BuildMaterialNode(PbrMaterialElement, *MaterialNode))
				{
					MaterialNode = nullptr;
				}

				BaseNode = MaterialNode;
			}
			else if (MaterialElement->IsA(EDatasmithElementType::DecalMaterial))
			{
				IDatasmithDecalMaterialElement& DecalMaterialElement = static_cast<IDatasmithDecalMaterialElement&>(*MaterialElement);

				UInterchangeDecalMaterialNode* MaterialNode = NewObject<UInterchangeDecalMaterialNode>(&NodeContainer);
				const FString MaterialNodeUid = NodeUtils::DecalMaterialPrefix + FDatasmithUtils::SanitizeObjectName(MaterialElement->GetName());
				NodeContainer.SetupNode(MaterialNode, MaterialNodeUid, *FDatasmithUtils::SanitizeObjectName(MaterialElement->GetLabel()), EInterchangeNodeContainerType::TranslatedAsset);

				if (!BuildMaterialNode(DecalMaterialElement, *MaterialNode))
				{
					MaterialNode = nullptr;
				}

				BaseNode = MaterialNode;
			}

			if (BaseNode)
			{
				BaseNode->SetAssetName(FDatasmithUtils::SanitizeObjectName(MaterialElement->GetName()));
			}
		}

		return BaseNode;
	}
}

