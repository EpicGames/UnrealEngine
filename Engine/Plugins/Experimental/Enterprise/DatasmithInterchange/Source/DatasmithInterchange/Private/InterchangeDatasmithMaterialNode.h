// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithDefinitions.h"

#include "Nodes/InterchangeBaseNode.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeShaderGraphNode.h"

#include "Engine/EngineTypes.h"

#include "InterchangeDatasmithMaterialNode.generated.h"

class IDatasmithBaseMaterialElement;
class UInterchangeBaseNodeContainer;

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithPbrMaterialNode : public UInterchangeShaderGraphNode
{
	GENERATED_BODY()

public:

	UInterchangeDatasmithPbrMaterialNode()
	{
		MaterialFunctionsDependencies.Initialize(Attributes, MaterialFunctionsDependenciesKey);
	}

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("DatasmithPbrMaterialNode");
		return TypeName;
	}

	bool GetCustomShadingModel(EMaterialShadingModel& AttributeValue) const
	{
		int32 LocalAttributeValue;
		if (GetInt32Attribute(ShadingModelAttrName, LocalAttributeValue))
		{
			AttributeValue = static_cast<EMaterialShadingModel>(LocalAttributeValue);
			return true;
		}

		return false;
	}

	bool SetCustomShadingModel(EMaterialShadingModel AttributeValue)
	{
		return AddInt32Attribute(ShadingModelAttrName, static_cast<int32>(AttributeValue));
	}

	bool GetCustomBlendMode(EBlendMode& AttributeValue) const
	{
		int32 LocalAttributeValue;
		if (GetInt32Attribute(BlendModeAttrName, LocalAttributeValue))
		{
			AttributeValue = static_cast<EBlendMode>(LocalAttributeValue);
			return true;
		}

		return false;
	}

	bool SetCustomBlendMode(EBlendMode AttributeValue)
	{
		return AddInt32Attribute(BlendModeAttrName, static_cast<int32>(AttributeValue));
	}

	bool GetCustomOpacityMaskClipValue(float& AttributeValue) const
	{
		return GetFloatAttribute(OpacityMaskClipValueAttrName, AttributeValue);
	}

	bool SetCustomOpacityMaskClipValue(float AttributeValue)
	{
		return AddFloatAttribute(OpacityMaskClipValueAttrName, AttributeValue);
	}

	bool GetCustomTranslucencyLightingMode(ETranslucencyLightingMode& AttributeValue) const
	{
		int32 LocalAttributeValue;
		if (GetInt32Attribute(TranslucencyLightingModeAttrName, LocalAttributeValue))
		{
			AttributeValue = static_cast<ETranslucencyLightingMode>(LocalAttributeValue);
			return true;
		}

		return false;
	}

	bool SetCustomTranslucencyLightingMode(ETranslucencyLightingMode AttributeValue)
	{
		return AddInt32Attribute(TranslucencyLightingModeAttrName, static_cast<int32>(AttributeValue));
	}

	void GetMaterialFunctionsDependencies(TArray<FString>& MaterialFunctionUids)
	{
		MaterialFunctionsDependencies.GetItems(MaterialFunctionUids);
	}

	bool AddMaterialFunctionsDependency(const FString& MaterialFunctionUid)
	{
		return MaterialFunctionsDependencies.AddItem(MaterialFunctionUid);
	}

	bool RemoveAllMaterialFunctionsDependencies()
	{
		return MaterialFunctionsDependencies.RemoveAllItems();
	}

private:
	static const FString ShadingModelAttrName;
	static const FString BlendModeAttrName;
	static const FString OpacityMaskClipValueAttrName;
	static const FString TranslucencyLightingModeAttrName;
	static const FString MaterialFunctionsDependenciesKey;

	UE::Interchange::TArrayAttributeHelper<FString> MaterialFunctionsDependencies;
};

namespace UE::DatasmithInterchange::MaterialUtils
{
	extern const FString DefaultOutputIndexAttrName;
	extern const FString MaterialFunctionPathAttrName;

	extern const FString MaterialTypeAttrName;
	extern const FString MaterialQualityAttrName;
	extern const FString MaterialParentAttrName;

	void ProcessMaterialElements(TArray<TSharedPtr<IDatasmithBaseMaterialElement>>& InOutMaterialElements);
	UInterchangeBaseNode* AddMaterialNode(const TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement, UInterchangeBaseNodeContainer& NodeContainer);
}
