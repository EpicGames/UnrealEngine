// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeDatasmithMaterialPipeline.h"

#include "InterchangeDatasmithMaterialNode.h"
#include "InterchangeDatasmithMaterialPipeline.h"
#include "InterchangeDatasmithUtils.h"
#include "InterchangeDecalMaterialNode.h"

#include "DatasmithMaterialElements.h"
#include "IDatasmithSceneElements.h"

#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "InterchangeManager.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeDecalMaterialFactoryNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeMaterialInstanceNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"

#include "Materials/MaterialExpressionMaterialFunctionCall.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectRedirector.h"

DEFINE_LOG_CATEGORY(LogInterchangeMaterialPipeline);

namespace UE::Interchange::MaterialUtils
{
	UClass* FindMaterialExpressionClass(const TCHAR* ClassName)
	{
		if (!ensure(ClassName))
		{
			return nullptr;
		}

		if (UClass* Result = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Datasmith FindClass")))
		{
			return Result;
		}

		if (UObjectRedirector* RenamedClassRedirector = FindFirstObject<UObjectRedirector>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("Datasmith FindClass")))
		{
			return CastChecked<UClass>(RenamedClassRedirector->DestinationObject);
		}

		return nullptr;
	}
}

UInterchangeDatasmithMaterialPipeline::UInterchangeDatasmithMaterialPipeline()
	: Super()
{
	MaterialImport = EInterchangeMaterialImportOption::ImportAsMaterials;
}

void UInterchangeDatasmithMaterialPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* NodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	using namespace UE::DatasmithInterchange;

	TArray<UInterchangeMaterialInstanceNode*> InstancedMaterials;
	TArray<UInterchangeDecalMaterialNode*> DecalMaterials;
	TArray<UInterchangeShaderNode*> ShaderNodes;

	//Find all translated node we need for this pipeline
	NodeContainer->IterateNodes([&](const FString& NodeUid, UInterchangeBaseNode* Node)
		{
			if (UInterchangeMaterialInstanceNode* MaterialNode = Cast<UInterchangeMaterialInstanceNode>(Node))
			{
				InstancedMaterials.Add(MaterialNode);
			}
			else if (UInterchangeDecalMaterialNode* DecalNode = Cast<UInterchangeDecalMaterialNode>(Node))
			{
				DecalMaterials.Add(DecalNode);
			}
			else if (UInterchangeShaderNode* ShaderNode = Cast<UInterchangeShaderNode>(Node))
			{
				ShaderNodes.Add(ShaderNode);
			}
		});

	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

	UpdateMaterialFactoryNodes(ShaderNodes);

	for (UInterchangeMaterialInstanceNode* MaterialNode : InstancedMaterials)
	{
		PreImportMaterialNode(NodeContainer, MaterialNode);
	}

	for (UInterchangeDecalMaterialNode* MaterialNode : DecalMaterials)
	{
		PreImportDecalMaterialNode(NodeContainer, MaterialNode);
	}
}

void UInterchangeDatasmithMaterialPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* NodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	if (UMaterialInterface* ImportedMaterial = Cast<UMaterialInterface>(CreatedAsset))
	{
		UInterchangeMaterialInstanceFactoryNode* InstanceFactoryNode = nullptr;
		NodeContainer->BreakableIterateNodesOfType<UInterchangeMaterialInstanceFactoryNode>([&ImportedMaterial, &InstanceFactoryNode](const FString& NodeUid, UInterchangeMaterialInstanceFactoryNode* Node)
		{
			FSoftObjectPath NodeReferenceObject;
			Node->GetCustomReferenceObject(NodeReferenceObject);
			if (NodeReferenceObject.TryLoad() == ImportedMaterial)
			{
				InstanceFactoryNode = Node;
				return true;
			}

			return false;
		});

		if (InstanceFactoryNode)
		{
			PostImportMaterialInstanceFactoryNode(NodeContainer, InstanceFactoryNode, ImportedMaterial);
		}

		return;
	}
}

void UInterchangeDatasmithMaterialPipeline::PreImportMaterialNode(UInterchangeBaseNodeContainer* NodeContainer, UInterchangeMaterialInstanceNode* MaterialNode)
{
	FString ParentPath;	

	if(!MaterialNode->GetCustomParent(ParentPath) || ParentPath.IsEmpty())
	{
		return;
	}

	using namespace UE::DatasmithInterchange;

	FString PackagePath;

	int32 MaterialType;
	if (MaterialNode->GetInt32Attribute(MaterialUtils::MaterialTypeAttrName, MaterialType))
	{
		if (EDatasmithReferenceMaterialType(MaterialType) == EDatasmithReferenceMaterialType::Custom && FPackageName::DoesPackageExist(ParentPath))
		{
			PackagePath = ParentPath;
		}
		else
		{
			FString Host(FDatasmithReferenceMaterialManager::Get().GetHostFromString(ParentPath));

			if (TSharedPtr<FDatasmithReferenceMaterialSelector> MaterialSelector = FDatasmithReferenceMaterialManager::Get().GetSelector(*Host))
			{
				PackagePath = MaterialSelector->GetMaterialPath(EDatasmithReferenceMaterialType(MaterialType));
			}
			else
			{
				//FText FailReason = FText::Format(LOCTEXT("NoSelectorForHost", "No Material selector found for Host {0}. Skipping material {1} ..."), FText::FromString(Host), FText::FromString(MaterialElement.GetName()));
				//ImportContext.LogError(FailReason);
			}
		}
	}

	if (PackagePath.IsEmpty())
	{
		return;
	}

	const FString MaterialNodeUid = UInterchangeMaterialInstanceFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(MaterialNode->GetUniqueID());

	UInterchangeMaterialInstanceFactoryNode* MaterialFactoryNode = NodeUtils::FindOrAddFactoryNode<UInterchangeMaterialInstanceFactoryNode>(MaterialNode, NodeContainer, MaterialNodeUid);

	// Set MaterialFactoryNode's display label to MaterialNode's uniqueID
	// to reconcile mesh's slot names and material assets
	MaterialFactoryNode->SetDisplayLabel(MaterialNode->GetAssetName());
	MaterialFactoryNode->SetCustomParent(PackagePath);

#if WITH_EDITOR
	const UClass* MaterialClass = FApp::IsGame() ? UMaterialInstanceDynamic::StaticClass() : UMaterialInstanceConstant::StaticClass();
	MaterialFactoryNode->SetCustomInstanceClassName(MaterialClass->GetPathName());
#else
	MaterialFactoryNode->SetCustomInstanceClassName(UMaterialInstanceDynamic::StaticClass()->GetPathName());
#endif

	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(MaterialNode, Inputs);

	for (const FString& InputName : Inputs)
	{
		FString InputValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);

		switch (UInterchangeShaderPortsAPI::GetInputType(MaterialNode, InputName))
		{
			case UE::Interchange::EAttributeTypes::Bool:
			{
				bool AttributeValue = false;
				MaterialNode->GetBooleanAttribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddBooleanAttribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::Int32:
			{
				int32 AttributeValue = 0;
				MaterialNode->GetInt32Attribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddInt32Attribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::Float:
			{
				float AttributeValue = 0.f;
				MaterialNode->GetFloatAttribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddFloatAttribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::LinearColor:
			{
				FLinearColor AttributeValue = FLinearColor::White;
				MaterialNode->GetLinearColorAttribute(InputValueKey, AttributeValue);
				MaterialFactoryNode->AddLinearColorAttribute(InputValueKey, AttributeValue);
			}
			break;
			case UE::Interchange::EAttributeTypes::String:
			{
				FString TextureName;
				MaterialNode->GetStringAttribute(InputValueKey, TextureName);

				if (!FPackageName::IsValidObjectPath(TextureName))
				{
					const FString TextureUid = UInterchangeTexture2DFactoryNode::GetTextureFactoryNodeUidFromTextureNodeUid(NodeUtils::TexturePrefix + TextureName);
					MaterialFactoryNode->AddStringAttribute(InputValueKey, TextureUid);
					MaterialFactoryNode->AddFactoryDependencyUid(TextureUid);
				}
				else
				{
					MaterialFactoryNode->AddStringAttribute(InputValueKey, TextureName);
				}
			}
			break;
		}
	}

	if (EDatasmithReferenceMaterialType(MaterialType) != EDatasmithReferenceMaterialType::Custom)
	{
		MaterialFactoryNode->AddStringAttribute(MaterialUtils::MaterialParentAttrName, ParentPath);
		MaterialFactoryNode->AddInt32Attribute(MaterialUtils::MaterialTypeAttrName, MaterialType);
		if(int32 MaterialQuality; MaterialNode->GetInt32Attribute(MaterialUtils::MaterialQualityAttrName, MaterialQuality))
		{
			MaterialFactoryNode->AddInt32Attribute(MaterialUtils::MaterialQualityAttrName, MaterialQuality);
		}
	}
}

void UInterchangeDatasmithMaterialPipeline::PostImportMaterialInstanceFactoryNode(const UInterchangeBaseNodeContainer* NodeContainer, UInterchangeMaterialInstanceFactoryNode* FactoryNode, UMaterialInterface* CreatedMaterial)
{
	using namespace UE::DatasmithInterchange;

#if WITH_EDITOR
	FString SelectorName;
	if (FactoryNode->GetStringAttribute(MaterialUtils::MaterialParentAttrName, SelectorName))
	{
		if (!SelectorName.IsEmpty())
		{
			const FString Host(FDatasmithReferenceMaterialManager::Get().GetHostFromString(SelectorName));

			int32 MaterialType;
			int32 MaterialQuality;

			ensure(FactoryNode->GetInt32Attribute(MaterialUtils::MaterialTypeAttrName, MaterialType));
			ensure(FactoryNode->GetInt32Attribute(MaterialUtils::MaterialQualityAttrName, MaterialQuality));

			if (TSharedPtr<FDatasmithReferenceMaterialSelector> MaterialSelector = FDatasmithReferenceMaterialManager::Get().GetSelector(*Host))
			{
				MaterialSelector->PostImportProcess((EDatasmithReferenceMaterialType)MaterialType, (EDatasmithReferenceMaterialQuality)MaterialQuality, Cast<UMaterialInstanceConstant>(CreatedMaterial));
			}
		}
	}
#endif

	// Process parameters which are connected to pre-existing textures
	TArray<FString> Inputs;
	UInterchangeShaderPortsAPI::GatherInputs(FactoryNode, Inputs);

	for (const FString& InputName : Inputs)
	{
		FString InputValueKey = UInterchangeShaderPortsAPI::MakeInputValueKey(InputName);

		if (UInterchangeShaderPortsAPI::GetInputType(FactoryNode, InputName) == UE::Interchange::EAttributeTypes::String)
		{
			const FName ParameterName = *InputName;
			UTexture* InstanceValue = nullptr;

			if (CreatedMaterial->GetTextureParameterValue(ParameterName, InstanceValue))
			{
				FString TextureName;
				FactoryNode->GetStringAttribute(InputValueKey, TextureName);

				if (FPackageName::IsValidObjectPath(TextureName))
				{
					if (UTexture* InputTexture = Cast<UTexture>(FSoftObjectPath(TextureName).TryLoad()))
					{
						if (InputTexture != InstanceValue)
						{
#if WITH_EDITOR
							if (UMaterialInstanceConstant* MaterialInstanceConstant = Cast<UMaterialInstanceConstant>(CreatedMaterial))
							{
								MaterialInstanceConstant->SetTextureParameterValueEditorOnly(ParameterName, InputTexture);
							}
							else
#endif // WITH_EDITOR
							if (UMaterialInstanceDynamic* MaterialInstanceDynamic = Cast<UMaterialInstanceDynamic>(CreatedMaterial))
							{
								MaterialInstanceDynamic->SetTextureParameterValue(ParameterName, InputTexture);
							}
						}
					}
				}
			}
		}
	}
}

void UInterchangeDatasmithMaterialPipeline::UpdateMaterialFactoryNodes(const TArray<UInterchangeShaderNode*>& ShaderNodes)
{
	using namespace UE::DatasmithInterchange;

	for (UInterchangeShaderNode* ShaderNode : ShaderNodes)
	{
		const FString FactoryNodeUid = UInterchangeMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(ShaderNode->GetUniqueID());
		UInterchangeFactoryBaseNode* FactoryNode = BaseNodeContainer->GetFactoryNode(FactoryNodeUid);
		if (!ensure(FactoryNode || bIdentifyDuplicateMaterials))
		{
			continue;
		}
		
		
		if (UInterchangeMaterialExpressionFactoryNode* MaterialExpressionFactoryNode = Cast<UInterchangeMaterialExpressionFactoryNode>(FactoryNode))
		{
			FString ExpressionClassName;
			MaterialExpressionFactoryNode->GetCustomExpressionClassName(ExpressionClassName);
			
			UClass* MaterialExpressionClass = UE::Interchange::MaterialUtils::FindMaterialExpressionClass(*ExpressionClassName);
			if (!MaterialExpressionClass)
			{
				UE_LOGF(LogInterchangeMaterialPipeline, Warning, "Invalid Material Expression Class.");
			}
			else
			{
				// This will transfer both porperties with delegates and transferable properties to the Factory Node.
				UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(ShaderNode, MaterialExpressionFactoryNode, false);
				UInterchangeUserDefinedAttributesAPI::AddApplyAndFillDelegatesToFactory(MaterialExpressionFactoryNode, MaterialExpressionClass);
			}
		}
		
		if (UInterchangeDatasmithPbrMaterialNode* PbrMaterialNode = Cast<UInterchangeDatasmithPbrMaterialNode>(ShaderNode))
		{
			UInterchangeBaseMaterialFactoryNode* BaseMaterialFactoryNode = Cast<UInterchangeBaseMaterialFactoryNode>(FactoryNode);
			ensure(BaseMaterialFactoryNode);

			TArray<FString> MaterialFunctionUids;
			PbrMaterialNode->GetMaterialFunctionsDependencies(MaterialFunctionUids);
			for (const FString& MaterialFunctionUid : MaterialFunctionUids)
			{
				const FString MaterialFunctionFactoryNodeUid = UInterchangeMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(MaterialFunctionUid);
				ensure(BaseNodeContainer->GetFactoryNode(MaterialFunctionFactoryNodeUid));
				BaseMaterialFactoryNode->AddFactoryDependencyUid(MaterialFunctionFactoryNodeUid);
			}

			PbrMaterialNode->RemoveAllMaterialFunctionsDependencies();

			if (UInterchangeMaterialFactoryNode* MaterialFactoryNode = Cast<UInterchangeMaterialFactoryNode>(FactoryNode))
			{
				EMaterialShadingModel ShadingModel;
				if (PbrMaterialNode->GetCustomShadingModel(ShadingModel))
				{
					MaterialFactoryNode->SetCustomShadingModel(ShadingModel);
				}

				EBlendMode BlendMode;
				if (PbrMaterialNode->GetCustomBlendMode(BlendMode))
				{
					MaterialFactoryNode->SetCustomBlendMode(BlendMode);
				}

				ETranslucencyLightingMode TranslucencyLightingMode;
				if (PbrMaterialNode->GetCustomTranslucencyLightingMode(TranslucencyLightingMode))
				{
					MaterialFactoryNode->SetCustomTranslucencyLightingMode(TranslucencyLightingMode);
				}

				// TODO: Add opacity mask clip value when implemented
				//float OpacityMaskClip;
				//if (PbrMaterialNode->GetCustomOpacityMaskClipValue(OpacityMaskClip))
				//{
				//	MaterialFactoryNode->SetCustomOpacityMaskClipValue(OpacityMaskClip);
				//}
			}
		}


	}
}

void UInterchangeDatasmithMaterialPipeline::PreImportDecalMaterialNode(UInterchangeBaseNodeContainer* NodeContainer, UInterchangeDecalMaterialNode* MaterialNode)
{
	using namespace UE::DatasmithInterchange;

	const FString MaterialNodeUid = UInterchangeDecalMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(MaterialNode->GetUniqueID());
	UInterchangeDecalMaterialFactoryNode* MaterialFactoryNode = NodeUtils::FindOrAddFactoryNode<UInterchangeDecalMaterialFactoryNode>(MaterialNode, NodeContainer, MaterialNodeUid);

	// Handle Diffuse Texture
	FString DiffuseTexturePath;
	if (MaterialNode->GetCustomDiffuseTexturePath(DiffuseTexturePath))
	{
		if (!FPackageName::IsValidObjectPath(DiffuseTexturePath))
		{
			const FString TextureUid = UInterchangeTexture2DFactoryNode::GetTextureFactoryNodeUidFromTextureNodeUid(NodeUtils::TexturePrefix + DiffuseTexturePath);
			MaterialFactoryNode->SetCustomDiffuseTexturePath(TextureUid);
			MaterialFactoryNode->AddFactoryDependencyUid(TextureUid);
		}
	}
	else
	{
		UE_LOGF(LogInterchangeMaterialPipeline, Warning, "Decal Material Node doesn't have Diffuse Texture Path Attribute set.");
	}

	// Handle Normal Texture
	FString NormalTexturePath;
	if (MaterialNode->GetCustomNormalTexturePath(NormalTexturePath))
	{
		if (!FPackageName::IsValidObjectPath(DiffuseTexturePath))
		{
			const FString TextureUid = UInterchangeTexture2DFactoryNode::GetTextureFactoryNodeUidFromTextureNodeUid(NodeUtils::TexturePrefix + DiffuseTexturePath);
			MaterialFactoryNode->SetCustomNormalTexturePath(TextureUid);
			MaterialFactoryNode->AddFactoryDependencyUid(TextureUid);
		}
	}
	else
	{
		UE_LOGF(LogInterchangeMaterialPipeline, Warning, "Decal Material Node doesn't have Normal Texture Path Attribute set.");
	}	
}
