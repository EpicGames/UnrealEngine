// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangeGenericMaterialPipeline.h"

#include "InterchangeDatasmithMaterialPipeline.generated.h"

class UMaterialInterface;
class UInterchangeDatasmithPbrMaterialNode;
class UInterchangeFactoryBaseNode;
class UInterchangeMaterialFactoryNode;
class UInterchangeMaterialInstanceFactoryNode;
class UInterchangeShaderGraphNode;
class UInterchangeMaterialInstanceNode;
class UInterchangeDecalMaterialNode;

DECLARE_LOG_CATEGORY_EXTERN(LogInterchangeMaterialPipeline, Log, All);

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithMaterialPipeline : public UInterchangeGenericMaterialPipeline
{
	GENERATED_BODY()

	UInterchangeDatasmithMaterialPipeline();

protected:
	virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;
	virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		// This pipeline creates UObjects and assets. Not safe to execute outside of main thread.
		return false;
	}

private:
	void PreImportMaterialNode(UInterchangeBaseNodeContainer* NodeContainer, UInterchangeMaterialInstanceNode* MaterialNode);
	void PostImportMaterialInstanceFactoryNode(const UInterchangeBaseNodeContainer* NodeContainer, UInterchangeMaterialInstanceFactoryNode* FactoryNode, UMaterialInterface* CreatedMaterial);
	void UpdateMaterialFactoryNodes(const TArray<UInterchangeShaderNode*>& ShaderNodes);
	void PreImportDecalMaterialNode(UInterchangeBaseNodeContainer* NodeContainer, UInterchangeDecalMaterialNode* MaterialNode);
};