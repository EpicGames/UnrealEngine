// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithPipeline.h"

#include "InterchangeDatasmithAreaLightNode.h"
#include "InterchangeDatasmithAreaLightFactoryNode.h"
#include "InterchangeDatasmithLevelPipeline.h"
#include "InterchangeDatasmithLog.h"
#include "InterchangeDatasmithMaterialPipeline.h"
#include "InterchangeDatasmithStaticMeshPipeline.h"
#include "InterchangeDatasmithTextureData.h"
#include "InterchangeDatasmithUtils.h"

#include "InterchangeGenericAnimationPipeline.h"
#include "InterchangeGenericScenesPipeline.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangeLevelSequenceFactoryNode.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeSceneImportAssetFactoryNode.h"
#include "InterchangeSceneVariantSetsFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeTexture2DFactoryNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureFactoryNode.h"
#include "InterchangeTextureLightProfileFactoryNode.h"
#include "InterchangeTextureLightProfileNode.h"
#include "InterchangeMeshActorFactoryNode.h"

#include "ExternalSource.h"
#include "DatasmithAreaLightActor.h"

#include "StaticMeshAttributes.h"
#include "Engine/StaticMeshActor.h"
#include "Components/StaticMeshComponent.h"

#define LOCTEXT_NAMESPACE "InterchangeDatasmithPipeline"

namespace UE::Interchange::StaticMeshUtils
{
#if WITH_EDITORONLY_DATA
	// Mirror of DatasmithMeshHelper::IsMeshValid
	bool HasValidTriangleData(const FMeshDescription& MeshDescription, const FMeshBuildSettings& BuildSettings)
	{
		const FVector BuildScale = BuildSettings.BuildScale3D;
		TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();
		FVector3f RawNormalScale(BuildScale.Y * BuildScale.Z, BuildScale.X * BuildScale.Z, BuildScale.X * BuildScale.Y); // Component-wise scale

		for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
		{
			TArrayView<const FVertexID> VertexIDs = MeshDescription.GetTriangleVertices(TriangleID);
			FVector3f Corners[3] =
			{
				VertexPositions[VertexIDs[0]],
				VertexPositions[VertexIDs[1]],
				VertexPositions[VertexIDs[2]]
			};

			FVector3f RawNormal = (Corners[1] - Corners[2]) ^ (Corners[0] - Corners[2]);
			RawNormal *= RawNormalScale;
			float FourSquaredTriangleArea = RawNormal.SizeSquared();

			// We support even small triangles, but this function is still useful to
			// see if we have at least one valid triangle in the mesh
			if (FourSquaredTriangleArea > 0.0f)
			{
				return true;
			}
		}

		// all faces are degenerated, mesh is invalid
		return false;
	}

	bool IsMeshValid(const UStaticMesh* StaticMesh)
	{
		// LOD index 0 has to be valid for a mesh to be valid.
		if (!StaticMesh->IsMeshDescriptionValid(0))
		{
			return false;
		}

		const FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
		const FMeshDescription& MeshDescription = *StaticMesh->GetMeshDescription(0);

		if (HasValidTriangleData(MeshDescription, BuildSettings))
		{
			return true;
		}

		return false;
	}
#endif
}

UInterchangeDatasmithPipeline::UInterchangeDatasmithPipeline()
{
	CommonSkeletalMeshesAndAnimationsProperties = CreateDefaultSubobject<UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties>("CommonSkeletalMeshesAndAnimationsProperties");
	
	CommonMeshesProperties = CreateDefaultSubobject<UInterchangeGenericCommonMeshesProperties>("CommonMeshesProperties");
	CommonMeshesProperties->bConvertStaticsInBoneHierarchyToSkeletals = false;
	CommonMeshesProperties->bBakeMeshes = false;

	MaterialPipeline = CreateDefaultSubobject<UInterchangeDatasmithMaterialPipeline>("DatasmithMaterialPipeline");
	MeshPipeline = CreateDefaultSubobject<UInterchangeDatasmithStaticMeshPipeline>("DatasmithMeshPipeline");
	LevelPipeline = CreateDefaultSubobject<UInterchangeDatasmithLevelPipeline>("DatasmithLevelPipeline");
	AnimationPipeline = CreateDefaultSubobject<UInterchangeGenericAnimationPipeline>("AnimationPipeline");

	MeshPipeline->CommonMeshesProperties = CommonMeshesProperties;
	MeshPipeline->CommonSkeletalMeshesAndAnimationsProperties = CommonSkeletalMeshesAndAnimationsProperties;

	AnimationPipeline->CommonMeshesProperties = CommonMeshesProperties;
	AnimationPipeline->CommonSkeletalMeshesAndAnimationsProperties = CommonSkeletalMeshesAndAnimationsProperties;
}

void UInterchangeDatasmithPipeline::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	Super::AdjustSettingsForContext(ContextParams);

	if (MaterialPipeline)
	{
		MaterialPipeline->AdjustSettingsForContext(ContextParams);
	}

	if (MeshPipeline)
	{
		MeshPipeline->AdjustSettingsForContext(ContextParams);
	}

	if (LevelPipeline)
	{
		LevelPipeline->AdjustSettingsForContext(ContextParams);
	}

	if (AnimationPipeline)
	{
		AnimationPipeline->AdjustSettingsForContext(ContextParams);
	}

	CacheContextParam = ContextParams;
}

void UInterchangeDatasmithPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
{
	using namespace UE::DatasmithInterchange;

	BaseNodeContainer = InBaseNodeContainer;

	ensure(BaseNodeContainer);
	ensure(Results);

	auto ExecutePreImportPipelineFunc = [this, &SourceDatas, &ContentBasePath](UInterchangePipelineBase* Pipeline)
	{
		if (Pipeline)
		{
			Pipeline->AdjustSettingsForContext(CacheContextParam);
			Pipeline->SetResultsContainer(this->Results);
			Pipeline->ScriptedExecutePipeline(this->BaseNodeContainer, SourceDatas, ContentBasePath);
		}
	};

	ExecutePreImportPipelineFunc(MaterialPipeline);
	for (UInterchangeTextureFactoryNode* TextureFactoryNode : NodeUtils::GetNodes<UInterchangeTextureFactoryNode>(BaseNodeContainer))
	{
		PreImportTextureFactoryNode(TextureFactoryNode);
	}

	ExecutePreImportPipelineFunc(MeshPipeline);

	if (CacheContextParam.ContextType == EInterchangePipelineContext::SceneImport || CacheContextParam.ContextType == EInterchangePipelineContext::SceneReimport)
	{
		ExecutePreImportPipelineFunc(LevelPipeline);
		ExecutePreImportPipelineFunc(AnimationPipeline);

		const FString PackageSubPath = FPaths::GetBaseFilename(SourceDatas[0]->GetFilename());

#if WITH_EDITORONLY_DATA
		LevelPipeline->SceneImportFactoryNode->SetCustomSubPath(PackageSubPath);
#endif

		// Textures
		for (UInterchangeTextureFactoryNode* TextureFactoryNode : NodeUtils::GetNodes<UInterchangeTextureFactoryNode>(BaseNodeContainer))
		{
			TextureFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Textures"));
			TextureFactoryNode->SetEnabled(true);
		}

		// Materials
		for (UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode : NodeUtils::GetNodes<UInterchangeBaseMaterialFactoryNode>(BaseNodeContainer))
		{
			if (bCreateMaterialReferencesFolders)
			{
				if (MaterialFactoryNode->IsA<UInterchangeMaterialFactoryNode>())
				{
					MaterialFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Materials/References"));
				}
				else if (MaterialFactoryNode->IsA<UInterchangeMaterialFunctionFactoryNode>())
				{
					MaterialFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Materials/References/Functions"));
				}
				else
				{
					MaterialFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Materials"));
				}
			}
			else
			{
				MaterialFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Materials"));
			}
			MaterialFactoryNode->SetEnabled(true);
		}

		// StaticMeshes
		for (UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode : NodeUtils::GetNodes<UInterchangeStaticMeshFactoryNode>(BaseNodeContainer))
		{
			StaticMeshFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Geometries"));
			StaticMeshFactoryNode->SetEnabled(true);
		}

		// LevelSequences
		for (UInterchangeLevelSequenceFactoryNode* AnimationTrackSetFactoryNode : NodeUtils::GetNodes<UInterchangeLevelSequenceFactoryNode>(BaseNodeContainer))
		{
			AnimationTrackSetFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Animations"));
			AnimationTrackSetFactoryNode->SetEnabled(true);
		}

		// LevelVariantSets
		for (UInterchangeSceneVariantSetsFactoryNode* LevelVariantSetFactoryNode : NodeUtils::GetNodes<UInterchangeSceneVariantSetsFactoryNode>(BaseNodeContainer))
		{
			LevelVariantSetFactoryNode->SetCustomSubPath(FPaths::Combine(PackageSubPath, "Variants"));
			LevelVariantSetFactoryNode->SetEnabled(true);
		}
	}
}

void UInterchangeDatasmithPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeDatasmithPipeline::ExecutePostImportPipeline);

	if (MaterialPipeline)
	{
		MaterialPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}

	if (MeshPipeline)
	{
		MeshPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
	}

	const bool bSceneImport = CacheContextParam.ContextType == EInterchangePipelineContext::SceneImport || CacheContextParam.ContextType == EInterchangePipelineContext::SceneReimport;
	if (bSceneImport)
	{
		if (LevelPipeline)
		{
			LevelPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
		}

		if (AnimationPipeline)
		{
			AnimationPipeline->ScriptedExecutePostImportPipeline(InBaseNodeContainer, NodeKey, CreatedAsset, bIsAReimport);
		}
	}
}

void UInterchangeDatasmithPipeline::PreImportTextureFactoryNode(UInterchangeTextureFactoryNode* TextureFactoryNode) const
{
	using namespace UE::DatasmithInterchange;

	TArray<FString> TargetNodes;
	TextureFactoryNode->GetTargetNodeUids(TargetNodes);
	if (TargetNodes.Num() == 0)
	{
		return;
	}

	const UInterchangeBaseNode* TargetNode = BaseNodeContainer->GetNode(TargetNodes[0]);
	if (!FInterchangeDatasmithTextureData::HasData(TargetNode))
	{
		return;
	}

	FInterchangeDatasmithTextureDataConst TextureData(TargetNode);
	TOptional< bool > bLocalFlipNormalMapGreenChannel;
	TOptional< TextureMipGenSettings > MipGenSettings;
	TOptional< TextureGroup > LODGroup;
	TOptional< TextureCompressionSettings > CompressionSettings;

	// Make sure to set the proper LODGroup as it's used to determine the CompressionSettings when using TEXTUREGROUP_WorldNormalMap
	EDatasmithTextureMode TextureMode;
	if (TextureData.GetCustomTextureMode(TextureMode))
	{
		switch (TextureMode)
		{
		case EDatasmithTextureMode::Diffuse:
			LODGroup = TEXTUREGROUP_World;
			break;
		case EDatasmithTextureMode::Specular:
			LODGroup = TEXTUREGROUP_WorldSpecular;
			break;
		case EDatasmithTextureMode::Bump:
		case EDatasmithTextureMode::Normal:
			LODGroup = TEXTUREGROUP_WorldNormalMap;
			CompressionSettings = TC_Normalmap;
			break;
		case EDatasmithTextureMode::NormalGreenInv:
			LODGroup = TEXTUREGROUP_WorldNormalMap;
			CompressionSettings = TC_Normalmap;
			bLocalFlipNormalMapGreenChannel = true;
			break;
		}
	}

	const TOptional< float > RGBCurve = [&TextureData]() -> TOptional< float >
		{
			float ElementRGBCurve;

			if (TextureData.GetCustomRGBCurve(ElementRGBCurve)
				&& FMath::IsNearlyEqual(ElementRGBCurve, 1.0f) == false
				&& ElementRGBCurve > 0.f)
			{
				return ElementRGBCurve;
			}

			return {};
		}();

	static_assert(TextureAddress::TA_Wrap == (int)EDatasmithTextureAddress::Wrap && TextureAddress::TA_Mirror == (int)EDatasmithTextureAddress::Mirror, "Texture Address enum doesn't match!");

	TOptional< TextureFilter > TexFilter;
	EDatasmithTextureFilter TextureFilter;
	if (TextureData.GetCustomTextureFilter(TextureFilter))
	{
		switch (TextureFilter)
		{
		case EDatasmithTextureFilter::Nearest:
			TexFilter = TextureFilter::TF_Nearest;
			break;
		case EDatasmithTextureFilter::Bilinear:
			TexFilter = TextureFilter::TF_Bilinear;
			break;
		case EDatasmithTextureFilter::Trilinear:
			TexFilter = TextureFilter::TF_Trilinear;
			break;
		}
	}

	TOptional< bool > bSrgb;
	EDatasmithColorSpace ColorSpace;
	if (TextureData.GetCustomSRGB(ColorSpace))
	{
		if (ColorSpace == EDatasmithColorSpace::sRGB)
		{
			bSrgb = true;
		}
		else if (ColorSpace == EDatasmithColorSpace::Linear)
		{
			bSrgb = false;
		}
	}

	EDatasmithTextureAddress AddressX;
	EDatasmithTextureAddress AddressY;
	UInterchangeTexture2DFactoryNode* Texture2DFactoryNode = Cast<UInterchangeTexture2DFactoryNode>(TextureFactoryNode);
	if (Texture2DFactoryNode
		&& TextureData.GetCustomTextureAddressX(AddressX)
		&& TextureData.GetCustomTextureAddressY(AddressY))
	{
		Texture2DFactoryNode->SetCustomAddressX((TextureAddress)AddressX);
		Texture2DFactoryNode->SetCustomAddressY((TextureAddress)AddressY);
	}

	if (bSrgb.IsSet())
	{
		TextureFactoryNode->SetCustomSRGB(bSrgb.GetValue());
	}

	if (bLocalFlipNormalMapGreenChannel.IsSet())
	{
		TextureFactoryNode->SetCustombFlipGreenChannel(bLocalFlipNormalMapGreenChannel.GetValue());
	}

	if (MipGenSettings.IsSet())
	{
		TextureFactoryNode->SetCustomMipGenSettings(MipGenSettings.GetValue());
	}

	if (LODGroup.IsSet())
	{
		TextureFactoryNode->SetCustomLODGroup(LODGroup.GetValue());
	}

	if (CompressionSettings.IsSet())
	{
		TextureFactoryNode->SetCustomCompressionSettings(CompressionSettings.GetValue());
	}

	if (RGBCurve.IsSet())
	{
		TextureFactoryNode->SetCustomAdjustRGBCurve(RGBCurve.GetValue());
	}

	if (TexFilter.IsSet())
	{
		TextureFactoryNode->SetCustomFilter(TexFilter.GetValue());
	}
}
#undef LOCTEXT_NAMESPACE