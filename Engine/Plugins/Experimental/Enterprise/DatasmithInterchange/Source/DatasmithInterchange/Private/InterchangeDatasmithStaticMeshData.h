// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithAdditionalData.h"
#include "DatasmithPayload.h"
#include "Nodes/InterchangeBaseNode.h"

#include "InterchangeDatasmithStaticMeshData.generated.h"

UCLASS()
class UDatasmithInterchangeStaticMeshDataNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	/**
	*/
	UPROPERTY()
	TMap<FString, TObjectPtr<UDatasmithAdditionalData>> AdditionalDataMap;
};
