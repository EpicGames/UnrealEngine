// Copyright Epic Games, Inc. All Rights Reserved. 

#include "InterchangeDatasmithStaticMeshPipeline.h"

#include "InterchangeDatasmithUtils.h"
#include "InterchangeDatasmithStaticMeshData.h"

#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeMeshNode.h"

#include "DatasmithAssetImportData.h"
#include "DatasmithImportOptions.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#ifdef NO_MONOLITHIC_BUILD
#include "DatasmithParametricSurfaceData.h"
#endif

void UInterchangeDatasmithStaticMeshPipeline::ExecutePipeline(UInterchangeBaseNodeContainer* NodeContainer, const TArray<UInterchangeSourceData*>& InSourceDatas, const FString& ContentBasePath)
{
	using namespace UE::DatasmithInterchange;

	Super::ExecutePipeline(NodeContainer, InSourceDatas, ContentBasePath);

	// Add material factory dependencies for meshes where all slots are filled with the same material
	for (UInterchangeStaticMeshFactoryNode* MeshFactoryNode : NodeUtils::GetNodes<UInterchangeStaticMeshFactoryNode>(NodeContainer))
	{
		TArray<FString> TargetNodes;
		MeshFactoryNode->GetTargetNodeUids(TargetNodes);
		if (TargetNodes.Num() == 0)
		{
			continue;
		}

		const UInterchangeMeshNode* MeshNode = Cast< UInterchangeMeshNode>(NodeContainer->GetNode(TargetNodes[0]));
		if (!MeshNode)
		{
			continue;
		}

		FString MaterialUid;
		if (!MeshNode->GetStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialUid))
		{
			continue;
		}

		const FString MaterialFactoryUid = UInterchangeMaterialFactoryNode::GetMaterialFactoryNodeUidFromMaterialNodeUid(MaterialUid);
		MeshFactoryNode->AddFactoryDependencyUid(MaterialFactoryUid);
		MeshFactoryNode->AddStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialFactoryUid);
	}
}

void UInterchangeDatasmithStaticMeshPipeline::ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* NodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
{
	using namespace UE::DatasmithInterchange;

	if (!NodeContainer || !CreatedAsset)
	{
		return;
	}

	Super::ExecutePostImportPipeline(NodeContainer, FactoryNodeKey, CreatedAsset, bIsAReimport);

	// If applicable, update FStaticMaterial of newly create mesh
	UStaticMesh* StaticMesh = Cast<UStaticMesh>(CreatedAsset);
	if (!StaticMesh)
	{
		return;
	}

	const UInterchangeStaticMeshFactoryNode* FactoryNode = Cast< UInterchangeStaticMeshFactoryNode>(NodeContainer->GetFactoryNode(FactoryNodeKey));
	if (!FactoryNode)
	{
		return;
	}

	ApplyMaterials(NodeContainer, FactoryNode, StaticMesh);
	ApplyAdditionalData(NodeContainer, FactoryNode, StaticMesh);
}

void UInterchangeDatasmithStaticMeshPipeline::ApplyMaterials(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeStaticMeshFactoryNode* FactoryNode, UStaticMesh* StaticMesh) const
{
	using namespace UE::DatasmithInterchange;

	FString MaterialFactoryUid;
	if (!FactoryNode->GetStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialFactoryUid))
	{
		return;
	}

	const UInterchangeBaseMaterialFactoryNode* MaterialFactoryNode = Cast< UInterchangeBaseMaterialFactoryNode>(NodeContainer->GetFactoryNode(MaterialFactoryUid));
	if (!MaterialFactoryNode)
	{
		return;
	}

	FSoftObjectPath MaterialFactoryNodeReferenceObject;
	MaterialFactoryNode->GetCustomReferenceObject(MaterialFactoryNodeReferenceObject);
	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(MaterialFactoryNodeReferenceObject.ResolveObject()))
	{
		TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
		for (FStaticMaterial& StaticMaterial : StaticMaterials)
		{
			StaticMaterial.MaterialInterface = MaterialInterface;
		}
	}
}

void UInterchangeDatasmithStaticMeshPipeline::ApplyAdditionalData(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeStaticMeshFactoryNode* FactoryNode, UStaticMesh* StaticMesh) const
{
#if WITH_EDITORONLY_DATA
	TArray<FString> TargetNodes;
	FactoryNode->GetTargetNodeUids(TargetNodes);
	if (TargetNodes.Num() == 0)
	{
		return;
	}

	const FString& MeshNodeUid = TargetNodes[0];

	TArray<UDatasmithInterchangeStaticMeshDataNode*> StaticMeshDataNodes;
	NodeContainer->GetNodesOfType<UDatasmithInterchangeStaticMeshDataNode>(StaticMeshDataNodes);
	if (StaticMeshDataNodes.Num() == 0)
	{
		return;
	}

	const UDatasmithInterchangeStaticMeshDataNode* StaticMeshDataNode = StaticMeshDataNodes[0];
	if (!StaticMeshDataNode)
	{
		return;
	}

#ifdef NO_MONOLITHIC_BUILD
	if (const TObjectPtr<UDatasmithAdditionalData>* AdditionalDataPtr = StaticMeshDataNode->AdditionalDataMap.Find(MeshNodeUid))
	{
		if (UDatasmithParametricSurfaceData* AdditionalData = Cast<UDatasmithParametricSurfaceData>(AdditionalDataPtr->Get()))
		{
			FDatasmithStaticMeshImportOptions StaticMeshOptions;
			FDatasmithAssetImportOptions AssetOptions;
			
			UDatasmithStaticMeshImportData::DefaultOptionsPair ImportOptions = UDatasmithStaticMeshImportData::DefaultOptionsPair(StaticMeshOptions, AssetOptions);

			UDatasmithStaticMeshImportData* MeshImportData = UDatasmithStaticMeshImportData::GetImportDataForStaticMesh(StaticMesh, ImportOptions);

			if (MeshImportData)
			{
				FString SourceUri;
				FMD5Hash SourceHash;
				if (SourceDatas.Num() > 0)
				{
					SourceUri = SourceDatas[0]->GetFilename();
					TOptional<FMD5Hash> OptHash = SourceDatas[0]->GetFileContentHash();
					if (OptHash.IsSet())
					{
						SourceHash = OptHash.GetValue();
					}
				}
				// Update the import data source file and set the mesh hash
				// #ueent_todo FH: piggybacking off of the SourceData file hash for now, until we have custom derived AssetImportData properly serialize to the AssetRegistry
				MeshImportData->Update(SourceUri, &SourceHash);
				MeshImportData->DatasmithImportInfo = FDatasmithImportInfo(SourceUri, SourceHash);

				// Set the final outer // #ueent_review: propagate flags of outer?
				AdditionalData->Rename(nullptr, MeshImportData);
				MeshImportData->AdditionalData.Add(AdditionalData);
			}
		}
	}
#endif
#endif
}
