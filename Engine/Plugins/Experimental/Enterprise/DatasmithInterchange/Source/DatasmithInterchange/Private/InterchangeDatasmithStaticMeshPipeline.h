// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangeGenericMeshPipeline.h"

#include "InterchangeDatasmithStaticMeshPipeline.generated.h"

class IDatasmithScene;
class UMaterialInterface;

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithStaticMeshPipeline : public UInterchangeGenericMeshPipeline
{
	GENERATED_BODY()

protected:
	virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;
	virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		// This pipeline creates UObjects and assets. Not safe to execute outside of main thread.
		return false;
	}

private:
	void ApplyMaterials(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeStaticMeshFactoryNode* FactoryNode, UStaticMesh* StaticMesh) const;
	void ApplyAdditionalData(const UInterchangeBaseNodeContainer* NodeContainer, const UInterchangeStaticMeshFactoryNode* FactoryNode, UStaticMesh* StaticMesh) const;
};