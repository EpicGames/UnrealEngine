// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithTextureData.h"

#include "InterchangeTextureNode.h"
#include "IDatasmithSceneElements.h"

#define IMPLEMENT_GET_ATTRIBUTE(Key) ConstBaseNode ? ConstBaseNode->GetAttribute(Key, AttributeValue) : false

#define IMPLEMENT_SET_ATTRIBUTE(Key) BaseNode ? BaseNode->SetAttribute(Key, AttributeValue) : false

namespace UE::DatasmithInterchange
{
	const FString FInterchangeDatasmithTextureDataConst::DatasmithTextureDataKey = TEXT("DatasmithTextureData::DatasmithTextureData");
	const FString FInterchangeDatasmithTextureDataConst::FileKey = TEXT("DatasmithTextureData::File");
	const FString FInterchangeDatasmithTextureDataConst::TextureModeKey = TEXT("DatasmithTextureData::TextureMode");
	const FString FInterchangeDatasmithTextureDataConst::TextureFilterKey = TEXT("DatasmithTextureData::TextureFilter");
	const FString FInterchangeDatasmithTextureDataConst::TextureAddressXKey = TEXT("DatasmithTextureData::TextureAddressX");
	const FString FInterchangeDatasmithTextureDataConst::TextureAddressYKey = TEXT("DatasmithTextureData::TextureAddressY");
	const FString FInterchangeDatasmithTextureDataConst::AllowResizeKey = TEXT("DatasmithTextureData::AllowResize");
	const FString FInterchangeDatasmithTextureDataConst::RGBCurveKey = TEXT("DatasmithTextureData::RGBCurve");
	const FString FInterchangeDatasmithTextureDataConst::SRGBKey = TEXT("DatasmithTextureData::SRGB");

	FInterchangeDatasmithTextureDataConst::FInterchangeDatasmithTextureDataConst(const UInterchangeBaseNode* InNode)
		: ConstBaseNode(InNode)
	{}

	FInterchangeDatasmithTextureData::FInterchangeDatasmithTextureData(UInterchangeBaseNode* InNode)
		: FInterchangeDatasmithTextureDataConst(InNode)
		, BaseNode(InNode)
	{
		BaseNode->AddBooleanAttribute(DatasmithTextureDataKey, true);
	}

	bool FInterchangeDatasmithTextureDataConst::HasData(const UInterchangeBaseNode* Node)
	{
		bool bTextureData;
		if (Node && Node->GetAttribute(DatasmithTextureDataKey, bTextureData) && bTextureData)
		{
			return true;
		}

		return false;
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomFile(FString& AttributeValue) const
	{
		return IMPLEMENT_GET_ATTRIBUTE(FileKey);
	}

	bool FInterchangeDatasmithTextureData::SetCustomFile(const FString& AttributeValue)
	{
		return IMPLEMENT_SET_ATTRIBUTE(FileKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomTextureMode(EDatasmithTextureMode& InAttributeValue) const
	{
		uint8 AttributeValue = 0;
		if (IMPLEMENT_GET_ATTRIBUTE(TextureModeKey))
		{
			InAttributeValue = static_cast<EDatasmithTextureMode>(AttributeValue);
			return true;
		}
		return false;
	}

	bool FInterchangeDatasmithTextureData::SetCustomTextureMode(const EDatasmithTextureMode& InAttributeValue)
	{
		uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
		return IMPLEMENT_SET_ATTRIBUTE(TextureModeKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomTextureFilter(EDatasmithTextureFilter& InAttributeValue) const
	{
		uint8 AttributeValue = 0;
		if (IMPLEMENT_GET_ATTRIBUTE(TextureFilterKey))
		{
			InAttributeValue = static_cast<EDatasmithTextureFilter>(AttributeValue);
			return true;
		}
		return false;
	}

	bool FInterchangeDatasmithTextureData::SetCustomTextureFilter(const EDatasmithTextureFilter& InAttributeValue)
	{
		uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
		return IMPLEMENT_SET_ATTRIBUTE(TextureFilterKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomTextureAddressX(EDatasmithTextureAddress& InAttributeValue) const
	{
		uint8 AttributeValue = 0;
		if (IMPLEMENT_GET_ATTRIBUTE(TextureAddressXKey))
		{
			InAttributeValue = static_cast<EDatasmithTextureAddress>(AttributeValue);
			return true;
		}
		return false;
	}

	bool FInterchangeDatasmithTextureData::SetCustomTextureAddressX(const EDatasmithTextureAddress& InAttributeValue)
	{
		uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
		return IMPLEMENT_SET_ATTRIBUTE(TextureAddressXKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomTextureAddressY(EDatasmithTextureAddress& InAttributeValue) const
	{
		uint8 AttributeValue = 0;
		if (IMPLEMENT_GET_ATTRIBUTE(TextureAddressYKey))
		{
			InAttributeValue = static_cast<EDatasmithTextureAddress>(AttributeValue);
			return true;
		}
		return false;
	}

	bool FInterchangeDatasmithTextureData::SetCustomTextureAddressY(const EDatasmithTextureAddress& InAttributeValue)
	{
		uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
		return IMPLEMENT_SET_ATTRIBUTE(TextureAddressYKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomAllowResize(bool& AttributeValue) const
	{
		return IMPLEMENT_GET_ATTRIBUTE(AllowResizeKey);
	}

	bool FInterchangeDatasmithTextureData::SetCustomAllowResize(const bool& AttributeValue)
	{
		return IMPLEMENT_SET_ATTRIBUTE(AllowResizeKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomRGBCurve(float& AttributeValue) const
	{
		return IMPLEMENT_GET_ATTRIBUTE(RGBCurveKey);
	}

	bool FInterchangeDatasmithTextureData::SetCustomRGBCurve(const float& AttributeValue)
	{
		return IMPLEMENT_SET_ATTRIBUTE(RGBCurveKey);
	}

	bool FInterchangeDatasmithTextureDataConst::GetCustomSRGB(EDatasmithColorSpace& InAttributeValue) const
	{
		uint8 AttributeValue = 0;
		if (IMPLEMENT_GET_ATTRIBUTE(SRGBKey))
		{
			InAttributeValue = static_cast<EDatasmithColorSpace>(AttributeValue);
			return true;
		}
		return false;
	}

	bool FInterchangeDatasmithTextureData::SetCustomSRGB(const EDatasmithColorSpace& InAttributeValue)
	{
		uint8 AttributeValue = static_cast<uint8>(InAttributeValue);
		return IMPLEMENT_SET_ATTRIBUTE(SRGBKey);
	}

	namespace TextureUtils
	{
		void ApplyTextureElementToNode(const TSharedRef<IDatasmithTextureElement>& TextureElement, UInterchangeBaseNode* BaseNode)
		{
			FInterchangeDatasmithTextureData TextureData(BaseNode);

			TextureData.SetCustomFile(TextureElement->GetFile());
			TextureData.SetCustomTextureMode(TextureElement->GetTextureMode());
			TextureData.SetCustomTextureFilter(TextureElement->GetTextureFilter());
			TextureData.SetCustomTextureAddressX(TextureElement->GetTextureAddressX());
			TextureData.SetCustomTextureAddressY(TextureElement->GetTextureAddressY());
			TextureData.SetCustomAllowResize(TextureElement->GetAllowResize());
			TextureData.SetCustomRGBCurve(TextureElement->GetRGBCurve());
			TextureData.SetCustomSRGB(TextureElement->GetSRGB());
		}
	}
}

#undef IMPLEMENT_SET_ATTRIBUTE
#undef IMPLEMENT_GET_ATTRIBUTE