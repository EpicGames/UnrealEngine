// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DatasmithDefinitions.h"
#include "Nodes/InterchangeBaseNode.h"

class IDatasmithTextureElement;
class UInterchangeTextureNode;

namespace UE::DatasmithInterchange
{
	/**
	 * Helper struct to get/set Datasmith texture attributes on a node.
	 */
	struct FInterchangeDatasmithTextureDataConst
	{
		static bool HasData(const UInterchangeBaseNode* Node);

		FInterchangeDatasmithTextureDataConst(const UInterchangeBaseNode* InNode);

		bool GetCustomFile(FString& AttributeValue) const;

		bool GetCustomTextureMode(EDatasmithTextureMode& AttributeValue) const;

		bool GetCustomTextureFilter(EDatasmithTextureFilter& AttributeValue) const;

		bool GetCustomTextureAddressX(EDatasmithTextureAddress& AttributeValue) const;

		bool GetCustomTextureAddressY(EDatasmithTextureAddress& AttributeValue) const;

		bool GetCustomAllowResize(bool& AttributeValue) const;

		bool GetCustomRGBCurve(float& AttributeValue) const;

		bool GetCustomSRGB(EDatasmithColorSpace& AttributeValue) const;

	protected:
		const UInterchangeBaseNode* ConstBaseNode;

		static const FString DatasmithTextureDataKey;
		static const FString FileKey;
		static const FString TextureModeKey;
		static const FString TextureFilterKey;
		static const FString TextureAddressXKey;
		static const FString TextureAddressYKey;
		static const FString AllowResizeKey;
		static const FString RGBCurveKey;
		static const FString SRGBKey;
	};

	struct FInterchangeDatasmithTextureData : public FInterchangeDatasmithTextureDataConst
	{
		FInterchangeDatasmithTextureData(UInterchangeBaseNode* InNode);

		bool SetCustomFile(const FString& AttributeValue);

		bool SetCustomTextureMode(const EDatasmithTextureMode& AttributeValue);

		bool SetCustomTextureFilter(const EDatasmithTextureFilter& AttributeValue);

		bool SetCustomTextureAddressX(const EDatasmithTextureAddress& AttributeValue);

		bool SetCustomTextureAddressY(const EDatasmithTextureAddress& AttributeValue);

		bool SetCustomAllowResize(const bool& AttributeValue);

		bool SetCustomRGBCurve(const float& AttributeValue);

		bool SetCustomSRGB(const EDatasmithColorSpace& AttributeValue);

	private:
		UInterchangeBaseNode* BaseNode;
	};

	namespace TextureUtils
	{
		/**
		 * Register the properties of the TextureElement into the BaseNode's attributes.
		 */
		void ApplyTextureElementToNode(const TSharedRef<IDatasmithTextureElement>& TextureElement, UInterchangeBaseNode* BaseNode);
	}
}