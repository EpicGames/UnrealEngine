// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithTranslator.h"

#include "InterchangeDatasmithAreaLightNode.h"
#include "InterchangeDatasmithLog.h"
#include "InterchangeDatasmithMaterialNode.h"
#include "InterchangeDatasmithStaticMeshData.h"
#include "InterchangeDatasmithTextureData.h"
#include "InterchangeDatasmithUtils.h"

#include "DatasmithAnimationElements.h"
#include "DatasmithMaterialElements.h"
#include "DatasmithScene.h"
#include "DatasmithSceneSource.h"
#include "DatasmithTranslatableSource.h"
#include "DatasmithTranslatorManager.h"
#include "DatasmithUtils.h"
#include "DatasmithVariantElements.h"
#include "IDatasmithSceneElements.h"
#ifdef NO_MONOLITHIC_BUILD
#include "DatasmithParametricSurfaceData.h"

#include "CADOptions.h"
#endif
#include "ExternalSourceModule.h"
#include "SourceUri.h"
#include "InterchangeCameraNode.h"
#include "InterchangeAnimationTrackSetNode.h"
#include "InterchangeLightNode.h"
#include "InterchangeDecalNode.h"
#include "InterchangeManager.h"
#include "InterchangeMaterialDefinitions.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeSceneNode.h"
#include "InterchangeTexture2DNode.h"
#include "InterchangeTextureLightProfileNode.h"
#include "InterchangeTextureLightProfileFactoryNode.h"
#include "InterchangeTranslatorHelper.h"
#include "InterchangeVariantSetNode.h"
#include "StaticMeshOperations.h"
#include "Nodes/InterchangeSourceNode.h"

#include "Async/ParallelFor.h"
#include "HAL/ConsoleManager.h"
#include "Misc/App.h"
#include "Misc/PackageName.h"

#if WITH_EDITOR
#include "DesktopPlatformModule.h"
#include "Dialogs/DlgPickPath.h"
#include "IDesktopPlatform.h"
#include "Interfaces/IMainFrameModule.h"
#include "ObjectTools.h"
#include "Styling/SlateIconFinder.h"
#include "UI/DatasmithImportOptionsWindow.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "DatasmithInterchange"

static TSet<FString> ExcludedFormats{
	TEXT("catpart"),
	TEXT("catproduct"),
	TEXT("catshape"),
	TEXT("cgr"),
	TEXT("3dxml"),
	TEXT("3drep"),
	TEXT("model"),
	TEXT("session"),
	TEXT("exp"),
	TEXT("dlv"),
	TEXT("asm.*s"),
	TEXT("creo.*"),
	TEXT("creo"),
	TEXT("neu.*"),
	TEXT("neu"),
	TEXT("prt.*"),
	TEXT("xas"),
	TEXT("xpr"),
	TEXT("iam"),
	TEXT("ipt"),
	TEXT("iges"),
	TEXT("igs"),
	TEXT("jt"),
	TEXT("sat"),
	TEXT("sab"),
	TEXT("sldasm"),
	TEXT("sldprt"),
	TEXT("step"),
	TEXT("stp"),
	TEXT("stpz"),
	TEXT("stpx"),
	TEXT("stpxz"),
	TEXT("xml"),
	TEXT("x_t"),
	TEXT("x_b"),
	TEXT("xmt"),
	TEXT("xmt_txt"),
	TEXT("asm"),
	TEXT("prt"),
	TEXT("par"),
	TEXT("psm"),
	TEXT("dwg"),
	TEXT("dxf"),
	TEXT("ifc"),
	TEXT("ifczip"),
	TEXT("hsf"),
	TEXT("prc"),
	TEXT("3mf"),
	TEXT("3ds"),
	TEXT("dae"),
	TEXT("dwf"),
	TEXT("dwfx"),
	TEXT("nwd"),
	TEXT("mf1"),
	TEXT("arc"),
	TEXT("unv"),
	TEXT("pkg"),
	TEXT("dgn"),  // available with Hoops Exchange 2023
	TEXT("stl"),
	TEXT("u3d"),
	TEXT("vda"),
	TEXT("vrml"),
	TEXT("wrl"),
	TEXT("wire"),
	TEXT("3dm"),
};

FRWLock UInterchangeDatasmithTranslator::StaticMeshDataNodeLock;

TArray<FString> UInterchangeDatasmithTranslator::GetSupportedFormats() const
{
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.CAD"), false);
	bool bInterchangeCADEnabled = CVar ? CVar->GetBool() : false;

	const TArray<FString> DatasmithFormats = FDatasmithTranslatorManager::Get().GetSupportedFormats();
	TArray<FString> Formats;
	Formats.Reserve(DatasmithFormats.Num() - 1);

	for (const FString& Format : DatasmithFormats)
	{
		if (Format.Contains(TEXT("gltf")) || Format.Contains(TEXT("glb")) || Format.Contains(TEXT("fbx")))
		{
			continue;
		}

		if (bInterchangeCADEnabled && ExcludedFormats.Contains(Format.ToLower()))
		{
			continue;
		}

		Formats.Add(Format);
	}

	return Formats;
}

bool UInterchangeDatasmithTranslator::CanImportSourceData(const UInterchangeSourceData* InSourceData) const
{
	using namespace UE::DatasmithImporter;

	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("Interchange.FeatureFlags.Import.CAD"), false);
	bool bInterchangeCADEnabled = CVar ? CVar->GetBool() : false;

	const FString FilePath = InSourceData->GetFilename();
	const FString FileExtension = FPaths::GetExtension(FilePath);
	if (FileExtension.Equals(TEXT("gltf"), ESearchCase::IgnoreCase) || FileExtension.Equals(TEXT("glb"), ESearchCase::IgnoreCase) || FileExtension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase))
	{
		// Do not translate gltf since there is already a native gltf interchange translator. 
		return false;
	}

	if (bInterchangeCADEnabled && ExcludedFormats.Contains(FileExtension.ToLower()))
	{
		return false;
	}

	const FSourceUri FileNameUri = FSourceUri::FromFilePath(FilePath);
	TSharedPtr<FExternalSource> ExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FileNameUri);

	return ExternalSource.IsValid() && ExternalSource->IsAvailable();
}

bool UInterchangeDatasmithTranslator::Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
{
	using namespace UE::DatasmithImporter;
	using namespace UE::DatasmithInterchange;

	// TODO: This code should eventually go into UInterchangeTranslatorBase once the FExternalSource module gets integrated into Interchange
	FString FilePath = FPaths::ConvertRelativePathToFull(SourceData->GetFilename());
	FileName = FPaths::GetCleanFilename(FilePath);
	const FSourceUri FileNameUri = FSourceUri::FromFilePath(FilePath);
	LoadedExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FileNameUri);

	if (!LoadedExternalSource.IsValid() || !LoadedExternalSource->IsAvailable())
	{
		return false;
	}

	StartTime = FPlatformTime::Cycles64();
	FPaths::NormalizeFilename(FilePath);

	TSharedPtr<IDatasmithScene> DatasmithScene;
	{
#ifdef NO_MONOLITHIC_BUILD
		TGuardValue<bool> EnableCADCache(CADLibrary::FImportParameters::bGEnableCADCache, true);
#endif
		if (GetSettings())
		{
#ifdef NO_MONOLITHIC_BUILD
			CADLibrary::FImportParameters::bGEnableCADCache = true;
#endif
			const TSharedPtr<IDatasmithTranslator>& DatasmithTranslator = LoadedExternalSource->GetAssetTranslator();
			if (DatasmithTranslator)
			{
				FDatasmithTranslatorCapabilities Capabilities;
				DatasmithTranslator->Initialize(Capabilities);

				if (!Capabilities.bParallelLoadStaticMeshSupported)
				{
					AsyncMode = EAsyncExecution::TaskGraphMainThread;
				}

				DatasmithTranslator->SetSceneImportOptions({ CachedSettings->DatasmithOption });
				CachedSettings->DatasmithOption->SaveConfig();
			}
		}

		// Should it be mutable instead? If Translate is const should we really be doing this?.
		DatasmithScene = LoadedExternalSource->TryLoad();

		if (!DatasmithScene.IsValid())
		{
			return false;
		}

	}

	{
		// File Creator Meta Data Information
		if (UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(&BaseNodeContainer))
		{
			using namespace UE::Interchange;
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVendorExtraInfoKey(), DatasmithScene->GetVendor());
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey(), DatasmithScene->GetProductName());
			SourceNode->SetExtraInformation(FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey(), DatasmithScene->GetProductVersion());
		}
	}

	// Add container for static mesh's additional data
	{
		const FString SceneName = DatasmithScene->GetName();
		const FString StaticMeshDataNodeUid = NodeUtils::ScenePrefix + SceneName + TEXT("_AdditionalData");

		StaticMeshDataNode = NewObject<UDatasmithInterchangeStaticMeshDataNode>(&BaseNodeContainer);

		BaseNodeContainer.SetupNode(StaticMeshDataNode.Get(), StaticMeshDataNodeUid, TEXT("StaticMesh_AdditonalData"), EInterchangeNodeContainerType::TranslatedAsset);
	}

	// Texture Nodes
	{
		FDatasmithUniqueNameProvider TextureNameProvider;

		for (int32 TextureIndex = 0, TextureNum = DatasmithScene->GetTexturesCount(); TextureIndex < TextureNum; ++TextureIndex)
		{
			if (TSharedPtr<IDatasmithTextureElement> TextureElement = DatasmithScene->GetTexture(TextureIndex))
			{
				const bool bIsIesProfile = FPaths::GetExtension(TextureElement->GetFile()).Equals(TEXT("ies"), ESearchCase::IgnoreCase);
				UClass* TextureClass = bIsIesProfile ? UInterchangeTextureLightProfileNode::StaticClass() : UInterchangeTexture2DNode::StaticClass();

				UInterchangeTextureNode* TextureNode = NewObject<UInterchangeTextureNode>(&BaseNodeContainer, TextureClass);

				const FString TextureNodeUid = NodeUtils::TexturePrefix + TextureElement->GetName();
				const FString DisplayLabel = TextureNameProvider.GenerateUniqueName(TextureElement->GetLabel());

				BaseNodeContainer.SetupNode(TextureNode, TextureNodeUid, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);

				if (bIsIesProfile)
				{
					TextureNode->SetPayLoadKey(TextureElement->GetFile());
				}
				else
				{
					TextureUtils::ApplyTextureElementToNode(TextureElement.ToSharedRef(), TextureNode);
					TextureNode->SetPayLoadKey(LexToString(TextureIndex));
				}
			}
		}
	}

	// Materials
	{
		FDatasmithUniqueNameProvider MaterialsNameProvider;
		const TCHAR* HostName = DatasmithScene->GetHost();

		TArray<TSharedPtr<IDatasmithBaseMaterialElement>> MaterialElements;
		MaterialElements.Reserve(DatasmithScene->GetMaterialsCount());

		for (int32 MaterialIndex = 0, MaterialNum = DatasmithScene->GetMaterialsCount(); MaterialIndex < MaterialNum; ++MaterialIndex)
		{
			if (TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = DatasmithScene->GetMaterial(MaterialIndex))
			{
				MaterialElements.Add(MaterialElement);
			}
		}

		MaterialUtils::ProcessMaterialElements(MaterialElements);

		for (TSharedPtr<IDatasmithBaseMaterialElement>& MaterialElement : MaterialElements)
		{
			if (UInterchangeBaseNode* MaterialNode = MaterialUtils::AddMaterialNode(MaterialElement, BaseNodeContainer))
			{
				const FString DisplayLabel = MaterialsNameProvider.GenerateUniqueName(MaterialNode->GetDisplayLabel());
				MaterialNode->SetDisplayLabel(DisplayLabel);

				if (MaterialElement->IsA(EDatasmithElementType::MaterialInstance))
				{
					UInterchangeMaterialInstanceNode* ReferenceMaterialNode = Cast<UInterchangeMaterialInstanceNode>(MaterialNode);
					if (int32 MaterialType; ReferenceMaterialNode->GetInt32Attribute(MaterialUtils::MaterialTypeAttrName, MaterialType) && EDatasmithReferenceMaterialType(MaterialType) == EDatasmithReferenceMaterialType::Custom)
					{
						ReferenceMaterialNode->SetCustomParent(static_cast<IDatasmithMaterialInstanceElement&>(*MaterialElement).GetCustomMaterialPathName());
					}
					else
					{
						ReferenceMaterialNode->SetCustomParent(HostName);
					}
				}
			}
		}
	}

	// Static Meshes
	{
		FDatasmithUniqueNameProvider StaticMeshNameProvider;
		for (int32 MeshIndex = 0, MeshNum = DatasmithScene->GetMeshesCount(); MeshIndex < MeshNum; ++MeshIndex)
		{
			if (const TSharedPtr<IDatasmithMeshElement> MeshElement = DatasmithScene->GetMesh(MeshIndex))
			{
				UInterchangeMeshNode* MeshNode = NewObject<UInterchangeMeshNode>(&BaseNodeContainer);
				const FString MeshNodeUid = NodeUtils::MeshPrefix + MeshElement->GetName();
				const FString DisplayLabel = StaticMeshNameProvider.GenerateUniqueName(MeshElement->GetLabel());

				BaseNodeContainer.SetupNode(MeshNode, MeshNodeUid, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
				MeshNode->SetPayLoadKey(LexToString(MeshIndex), EInterchangeMeshPayLoadType::STATIC);
				MeshNode->SetSkinnedMesh(false);
				MeshNode->SetCustomHasVertexNormal(true);
				// TODO: Interchange expect each LOD to have its own mesh node and to declare the number of vertices, however we don't know the content of a datasmith mesh until its bulk data is loaded.
				//       It is not clear what would be the proper way to properly translate the content of the Datasmith meshes without	loading all this data during the translation phase (done on the main thread).

				TSharedPtr<IDatasmithMaterialIDElement> GlobalMaterialID;
				for (int32 SlotIndex = 0, SlotCount = MeshElement->GetMaterialSlotCount(); SlotIndex < SlotCount; ++SlotIndex)
				{
					if (TSharedPtr<IDatasmithMaterialIDElement> MaterialID = MeshElement->GetMaterialSlotAt(SlotIndex))
					{
						if (MaterialID->GetId() == -1)
						{
							GlobalMaterialID = MaterialID;
							break;
						}
					}
				}

				if (GlobalMaterialID)
				{
					// Set dedicated attribute with value of material Uid.
					// Corresponding factory then mesh asset will be updated accordingly pre then post import in the pipeline
					const FString MaterialUid = NodeUtils::MaterialPrefix + GlobalMaterialID->GetName();
					MeshNode->AddStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialUid);
				}
				else
				{
					for (int32 SlotIndex = 0, SlotCount = MeshElement->GetMaterialSlotCount(); SlotIndex < SlotCount; ++SlotIndex)
					{
						if (const TSharedPtr<IDatasmithMaterialIDElement> MaterialID = MeshElement->GetMaterialSlotAt(SlotIndex))
						{
							const FString MaterialUid = NodeUtils::MaterialPrefix + MaterialID->GetName();
							if (BaseNodeContainer.GetNode(MaterialUid) != nullptr)
							{
								MeshNode->SetSlotMaterialDependencyUid(*FString::FromInt(MaterialID->GetId()), MaterialUid);
							}
						}
					}
				}
			}
		}
	}

	//Actors
	{
		// Add base scene node.
		UInterchangeSceneNode* SceneNode = NewObject< UInterchangeSceneNode >(&BaseNodeContainer);
		const FString SceneName = DatasmithScene->GetName();
		const FString SceneNodeUid = NodeUtils::ScenePrefix + SceneName;
		BaseNodeContainer.SetupNode(SceneNode, SceneNodeUid, DatasmithScene->GetLabel(), EInterchangeNodeContainerType::TranslatedScene);
		// TODO: This should be the instantiation of the DatasmithScene asset, and create a DatasmithSceneActor.

		for (int32 ActorIndex = 0, ActorNum = DatasmithScene->GetActorsCount(); ActorIndex < ActorNum; ++ActorIndex)
		{
			if (const TSharedPtr<IDatasmithActorElement> ActorElement = DatasmithScene->GetActor(ActorIndex))
			{
				HandleDatasmithActor(BaseNodeContainer, ActorElement.ToSharedRef(), SceneNode);
			}
		}
	}

	// Level sequences
	{
		const int32 SequencesCount = DatasmithScene->GetLevelSequencesCount();
		
		TArray<TSharedPtr<IDatasmithLevelSequenceElement>> LevelSequences;
		LevelSequences.Reserve(SequencesCount);


		for (int32 SequenceIndex = 0; SequenceIndex < SequencesCount; ++SequenceIndex)
		{
			TSharedPtr<IDatasmithLevelSequenceElement> SequenceElement = DatasmithScene->GetLevelSequence(SequenceIndex);
			if (!SequenceElement)
			{
				continue;
			}

			FDatasmithLevelSequencePayload LevelSequencePayload;
			LoadedExternalSource->GetAssetTranslator()->LoadLevelSequence(SequenceElement.ToSharedRef(), LevelSequencePayload);

			if (SequenceElement->GetAnimationsCount() > 0)
			{
				LevelSequences.Add(SequenceElement);
			}
		}

		AnimUtils::TranslateLevelSequences(LevelSequences, BaseNodeContainer, AnimationPayLoadMapping);
	}

	// Level variant sets
	// Note: Variants are not supported yet in game play mode
	if(!FApp::IsGame()) 
	{
		const int32 LevelVariantSetCount = DatasmithScene->GetLevelVariantSetsCount();

		TArray<TSharedPtr<IDatasmithLevelVariantSetsElement>> LevelVariantSets;
		LevelVariantSets.Reserve(LevelVariantSetCount);


		for (int32 LevelVariantSetIndex = 0; LevelVariantSetIndex < LevelVariantSetCount; ++LevelVariantSetIndex)
		{
			TSharedPtr<IDatasmithLevelVariantSetsElement> LevelVariantSetElement = DatasmithScene->GetLevelVariantSets(LevelVariantSetIndex);
			if (LevelVariantSetElement)
			{
				LevelVariantSets.Add(LevelVariantSetElement);
			}
		}

		VariantSetUtils::TranslateLevelVariantSets(LevelVariantSets, BaseNodeContainer);
	}

	// Log time spent to import incoming file in minutes and seconds
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	UE_LOGF(LogInterchangeDatasmith, Log, "Translation of %ls in[%d min %.3f s]", *FileName, ElapsedMin, ElapsedSeconds);

	return true;
}

void UInterchangeDatasmithTranslator::HandleDatasmithActor(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithActorElement>& ActorElement, const UInterchangeSceneNode* ParentNode) const
{
	using namespace UE::DatasmithInterchange;
	const FString NodeName = ActorElement->GetName();
	const FString ParentNodeUid = ParentNode->GetUniqueID();
	const FString NodeUid = NodeUtils::GetActorUid(*NodeName);

	UInterchangeSceneNode* InterchangeSceneNode = NewObject<UInterchangeSceneNode>(&BaseNodeContainer);
	BaseNodeContainer.SetupNode(InterchangeSceneNode, NodeUid, ActorElement->GetLabel(), EInterchangeNodeContainerType::TranslatedScene, ParentNodeUid);
	InterchangeSceneNode->SetAssetName(NodeName);

	const FTransform ActorTransform = ActorElement->GetRelativeTransform();
	InterchangeSceneNode->SetCustomLocalTransform(&BaseNodeContainer, ActorElement->GetRelativeTransform());
	// TODO: Layer association + component actors

	if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
	{
		TSharedRef<IDatasmithMeshActorElement> MeshActor = StaticCastSharedRef<IDatasmithMeshActorElement>(ActorElement);
		// TODO: GetStaticMeshPathName() might reference an asset that was not imported.
		const FString MeshUid = NodeUtils::MeshPrefix + MeshActor->GetStaticMeshPathName();
		InterchangeSceneNode->SetCustomAssetInstanceUid(MeshUid);

		TSharedPtr<IDatasmithMaterialIDElement> GlobalMaterialID;
		for (int32 OverrideIndex = 0, OverrideCount = MeshActor->GetMaterialOverridesCount(); OverrideIndex < OverrideCount; ++OverrideIndex)
		{
			if (const TSharedPtr<IDatasmithMaterialIDElement> MaterialID = MeshActor->GetMaterialOverride(OverrideIndex))
			{
				if (MaterialID->GetId() == -1)
				{
					GlobalMaterialID = MaterialID;
					break;
				}
			}
		}

		if (GlobalMaterialID)
		{
			// Set dedicated attribute with value of material Uid.
			// Corresponding factory then mesh actor will be updated accordingly pre then post import in the pipeline
			const FString MaterialUid = NodeUtils::MaterialPrefix + GlobalMaterialID->GetName();
			InterchangeSceneNode->AddStringAttribute(MeshUtils::MeshMaterialAttrName, MaterialUid);
		}
		else
		{
			for (int32 OverrideIndex = 0, OverrideCount = MeshActor->GetMaterialOverridesCount(); OverrideIndex < OverrideCount; ++OverrideIndex)
			{
				if (const TSharedPtr<IDatasmithMaterialIDElement> MaterialID = MeshActor->GetMaterialOverride(OverrideIndex))
				{
					const FString MaterialUid = NodeUtils::MaterialPrefix + FDatasmithUtils::SanitizeObjectName(MaterialID->GetName());
					if (BaseNodeContainer.GetNode(MaterialUid) != nullptr)
					{
						InterchangeSceneNode->SetSlotMaterialDependencyUid(*FString::FromInt(MaterialID->GetId()), MaterialUid);
					}
				}
			}
		}
	}
	else if (ActorElement->IsA(EDatasmithElementType::Camera))
	{
		TSharedRef<IDatasmithCameraActorElement> CameraActor = StaticCastSharedRef<IDatasmithCameraActorElement>(ActorElement);

		// We need to add camera asset node and then instance it in the scene node.
		UInterchangePhysicalCameraNode* CameraNode = AddCameraNode(BaseNodeContainer, CameraActor);
		InterchangeSceneNode->SetCustomAssetInstanceUid(CameraNode->GetUniqueID());
	}
	else if (ActorElement->IsA(EDatasmithElementType::Light))
	{
		TSharedRef<IDatasmithLightActorElement> LightActor = StaticCastSharedRef<IDatasmithLightActorElement>(ActorElement);

		// We need to add light asset node and then instance it in the scene node.
		UInterchangeBaseLightNode* LightNode = AddLightNode(BaseNodeContainer, LightActor);
		InterchangeSceneNode->SetCustomAssetInstanceUid(LightNode->GetUniqueID());
	}
	else if (ActorElement->IsA(EDatasmithElementType::Decal))
	{
		TSharedRef<IDatasmithDecalActorElement> DecalActor = StaticCastSharedRef<IDatasmithDecalActorElement>(ActorElement);

		UInterchangeDecalNode* DecalNode= AddDecalNode(BaseNodeContainer, DecalActor);
		InterchangeSceneNode->SetCustomAssetInstanceUid(DecalNode->GetUniqueID());
	}

	for (int32 ChildIndex = 0, ChildrenCount = ActorElement->GetChildrenCount(); ChildIndex < ChildrenCount; ++ChildIndex)
	{
		if (const TSharedPtr<IDatasmithActorElement> ChildActorElement = ActorElement->GetChild(ChildIndex))
		{
			HandleDatasmithActor(BaseNodeContainer, ChildActorElement.ToSharedRef(), InterchangeSceneNode);
		}
	}
}

UInterchangePhysicalCameraNode* UInterchangeDatasmithTranslator::AddCameraNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithCameraActorElement>& CameraActor) const
{
	using namespace UE::DatasmithInterchange;

	UInterchangePhysicalCameraNode* CameraNode = NewObject<UInterchangePhysicalCameraNode>(&BaseNodeContainer);
	const FString CameraUid = NodeUtils::CameraPrefix + CameraActor->GetName();
	BaseNodeContainer.SetupNode(CameraNode, CameraUid, CameraActor->GetLabel(), EInterchangeNodeContainerType::TranslatedAsset);

	CameraNode->SetCustomFocalLength(CameraActor->GetFocalLength());
	CameraNode->SetCustomSensorWidth(CameraActor->GetSensorWidth());
	const float SensorHeight = CameraActor->GetSensorWidth() / CameraActor->GetSensorAspectRatio();
	CameraNode->SetCustomSensorHeight(SensorHeight);

	// #cad_interchange:
	// TODO Add properties currently missing from the InterchangeCameraNode:
	//  - DepthOfField
	//  - FocusDistance
	//  - FStop
	//  - FocalLength
	//  - PostProcess
	//  - LookAtActor

	return CameraNode;
}

UInterchangeBaseLightNode* UInterchangeDatasmithTranslator::AddLightNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithLightActorElement>& LightActor) const
{
	using namespace UE::DatasmithInterchange;

	using FDatasmithLightUnits = std::underlying_type_t<EDatasmithLightUnits>;
	using FInterchangeLightUnits = std::underlying_type_t<EInterchangeLightUnits>;
	using FCommonLightUnits = std::common_type_t<FDatasmithLightUnits, FInterchangeLightUnits>;

	static_assert(FCommonLightUnits(EInterchangeLightUnits::Unitless) == FCommonLightUnits(EDatasmithLightUnits::Unitless), "EDatasmithLightUnits::Unitless differs from EInterchangeLightUnits::Unitless");
	static_assert(FCommonLightUnits(EInterchangeLightUnits::Lumens) == FCommonLightUnits(EDatasmithLightUnits::Lumens), "EDatasmithLightUnits::Lumens differs from EInterchangeLightUnits::Lumens");
	static_assert(FCommonLightUnits(EInterchangeLightUnits::Candelas) == FCommonLightUnits(EDatasmithLightUnits::Candelas), "EDatasmithLightUnits::Candelas differs from EInterchangeLightUnits::Candelas");
	static_assert(FCommonLightUnits(EInterchangeLightUnits::EV) == FCommonLightUnits(EDatasmithLightUnits::EV), "EDatasmithLightUnits::EV differs from EInterchangeLightUnits::EV");

	// TODO Add properties currently missing from the UInterchangeLightNode: everything
	UInterchangeBaseLightNode* LightNode = nullptr;
	if (LightActor->IsA(EDatasmithElementType::AreaLight))
	{
		const TSharedRef<IDatasmithAreaLightElement> AreaLightElement = StaticCastSharedRef<IDatasmithAreaLightElement>(LightActor);
		UInterchangeDatasmithAreaLightNode* AreaLightNode = NewObject<UInterchangeDatasmithAreaLightNode>(&BaseNodeContainer);

		const FString LightUid = NodeUtils::LightPrefix + LightActor->GetName();
		BaseNodeContainer.SetupNode(AreaLightNode, LightUid, LightActor->GetLabel(), EInterchangeNodeContainerType::TranslatedAsset);

		AreaLightNode->SetCustomLightType(static_cast<EDatasmithAreaLightActorType>(AreaLightElement->GetLightType()));
		AreaLightNode->SetCustomLightShape(static_cast<EDatasmithAreaLightActorShape>(AreaLightElement->GetLightShape()));
		AreaLightNode->SetCustomDimensions(FVector2D(AreaLightElement->GetLength(), AreaLightElement->GetWidth()));
		AreaLightNode->SetCustomIntensity(AreaLightElement->GetIntensity());
		AreaLightNode->SetCustomIntensityUnits(static_cast<EInterchangeLightUnits>(AreaLightElement->GetIntensityUnits()));
		AreaLightNode->SetCustomColor(AreaLightElement->GetColor());
		if (AreaLightElement->GetUseTemperature())
		{
			AreaLightNode->SetCustomTemperature(AreaLightElement->GetTemperature());
		}

		AreaLightNode->SetCustomSourceRadius(AreaLightElement->GetSourceRadius());
		AreaLightNode->SetCustomSourceLength(AreaLightElement->GetSourceLength());
		AreaLightNode->SetCustomAttenuationRadius(AreaLightElement->GetAttenuationRadius());
		AreaLightNode->SetCustomSpotlightInnerAngle(AreaLightElement->GetInnerConeAngle());
		AreaLightNode->SetCustomSpotlightOuterAngle(AreaLightElement->GetOuterConeAngle());

		LightNode = AreaLightNode;
		return LightNode;
	}
	else if (LightActor->IsA(EDatasmithElementType::SpotLight))
	{
		LightNode = NewObject<UInterchangeSpotLightNode>(&BaseNodeContainer);
	}
	else if (LightActor->IsA(EDatasmithElementType::LightmassPortal))
	{
		// TODO Add lightmass portal support in interchange.
		LightNode = NewObject<UInterchangeRectLightNode>(&BaseNodeContainer);
	}
	else if (LightActor->IsA(EDatasmithElementType::PointLight))
	{
		LightNode = NewObject<UInterchangePointLightNode>(&BaseNodeContainer);
	}
	else if (LightActor->IsA(EDatasmithElementType::DirectionalLight))
	{
		LightNode = NewObject<UInterchangeDirectionalLightNode>(&BaseNodeContainer);
	}
	else
	{
		ensure(false);
		LightNode = NewObject<UInterchangeLightNode>(&BaseNodeContainer);
	}

	ProcessIesProfile(BaseNodeContainer, *LightActor, Cast<UInterchangeLightNode>(LightNode));

	const FString LightUid = NodeUtils::LightPrefix + LightActor->GetName();
	BaseNodeContainer.SetupNode(LightNode, LightUid, LightActor->GetLabel(), EInterchangeNodeContainerType::TranslatedAsset);

	return LightNode;
}

void UInterchangeDatasmithTranslator::ProcessIesProfile(UInterchangeBaseNodeContainer& BaseNodeContainer, const IDatasmithLightActorElement& LightElement, UInterchangeLightNode* LightNode) const
{
	using namespace UE::DatasmithImporter;
	using namespace UE::DatasmithInterchange;

	if (!LightNode || !LightElement.GetUseIes())
	{
		return;
	}

	bool bUpdateLightNode = false;

	FString ProfileNodeUid = NodeUtils::TexturePrefix + LightElement.GetName() + TEXT("_IES");
	const FString DisplayLabel = FString(LightElement.GetName()) + TEXT("_IES");

	if (FPaths::FileExists(LightElement.GetIesTexturePathName()))
	{
		UInterchangeTextureNode* TextureNode = NewObject<UInterchangeTextureLightProfileNode>(&BaseNodeContainer);
		BaseNodeContainer.SetupNode(TextureNode, ProfileNodeUid, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
		bUpdateLightNode = true;
	}
	else if(FSoftObjectPath(LightElement.GetIesTexturePathName()).IsValid())
	{
		FString IESFactoryTextureId = UInterchangeFactoryBaseNode::BuildFactoryNodeUid(ProfileNodeUid);
		UInterchangeTextureLightProfileFactoryNode* FactoryNode = NewObject<UInterchangeTextureLightProfileFactoryNode>(&BaseNodeContainer);
		BaseNodeContainer.SetupNode(FactoryNode, IESFactoryTextureId, DisplayLabel, EInterchangeNodeContainerType::FactoryData);
		FactoryNode->SetCustomReferenceObject(FSoftObjectPath(LightElement.GetIesTexturePathName()));
		bUpdateLightNode = true;
	}
	else
	{
		const FString TextureNodeUid = NodeUtils::TexturePrefix + FDatasmithUtils::SanitizeObjectName(LightElement.GetIesTexturePathName());
		if (BaseNodeContainer.GetNode(TextureNodeUid))
		{
			ProfileNodeUid = TextureNodeUid;
			bUpdateLightNode = true;
		}
	}

	if (bUpdateLightNode)
	{
		LightNode->SetCustomIESTexture(ProfileNodeUid);
		LightNode->SetCustomUseIESBrightness(LightElement.GetUseIesBrightness());
		LightNode->SetCustomIESBrightnessScale(LightElement.GetIesBrightnessScale());
		LightNode->SetCustomRotation(LightElement.GetIesRotation().Rotator());
	}
}

UInterchangeDecalNode* UInterchangeDatasmithTranslator::AddDecalNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithDecalActorElement>& DecalActor) const
{
	using namespace UE::DatasmithInterchange;

	UInterchangeDecalNode* DecalNode = NewObject<UInterchangeDecalNode>(&BaseNodeContainer);
	const FString DecalUid = NodeUtils::DecalPrefix + DecalActor->GetName();
	BaseNodeContainer.SetupNode(DecalNode, DecalUid, DecalActor->GetLabel(), EInterchangeNodeContainerType::TranslatedAsset);

	DecalNode->SetCustomSortOrder(DecalActor->GetSortOrder());
	DecalNode->SetCustomDecalSize(DecalActor->GetDimensions());

	FString DecalMaterialPathName = DecalActor->GetDecalMaterialPathName();
	if(!FPackageName::IsValidObjectPath(DecalActor->GetDecalMaterialPathName()))
	{
		const FString DecalMaterialUid = NodeUtils::DecalMaterialPrefix + DecalActor->GetDecalMaterialPathName();
		if (BaseNodeContainer.IsNodeUidValid(DecalMaterialUid))
		{
			DecalMaterialPathName = DecalMaterialUid;
		}
	}
	DecalNode->SetCustomDecalMaterialPathName(DecalMaterialPathName);

	return DecalNode;
}

TOptional<UE::Interchange::FImportImage> UInterchangeDatasmithTranslator::GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	if (!LoadedExternalSource || !LoadedExternalSource->GetDatasmithScene())
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	int32 TextureIndex = 0;
	LexFromString(TextureIndex, *PayloadKey);
	TSharedPtr<IDatasmithScene> DatasmithScene = LoadedExternalSource->GetDatasmithScene();
	if (TextureIndex < 0 || TextureIndex >= DatasmithScene->GetTexturesCount())
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	TSharedPtr<IDatasmithTextureElement> TextureElement = DatasmithScene->GetTexture(TextureIndex);
	if (!TextureElement.IsValid())
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	UE::Interchange::Private::FScopedTranslator ScopedTranslator(TextureElement->GetFile(), Results, AnalyticsHandler);
	const IInterchangeTexturePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface< IInterchangeTexturePayloadInterface >();

	if (!ensure(TextureTranslator))
	{
		return TOptional<UE::Interchange::FImportImage>();
	}

	AlternateTexturePath = TextureElement->GetFile();

	return TextureTranslator->GetTexturePayloadData(PayloadKey, AlternateTexturePath);
}

TOptional<UE::Interchange::FImportLightProfile> UInterchangeDatasmithTranslator::GetLightProfilePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const
{
	if (!LoadedExternalSource || !LoadedExternalSource->GetDatasmithScene())
	{
		return TOptional<UE::Interchange::FImportLightProfile>();
	}

	UInterchangeSourceData* PayloadSourceData = UInterchangeManager::GetInterchangeManager().CreateSourceData(PayloadKey);
	UE::Interchange::Private::FScopedTranslator ScopedTranslator(PayloadKey, Results, AnalyticsHandler);
	const IInterchangeTextureLightProfilePayloadInterface* TextureTranslator = ScopedTranslator.GetPayLoadInterface< IInterchangeTextureLightProfilePayloadInterface >();
	if (!ensure(TextureTranslator))
	{
		return TOptional<UE::Interchange::FImportLightProfile>();
	}

	AlternateTexturePath = PayloadKey;

	AlternateTexturePath = PayloadKey;

	return TextureTranslator->GetLightProfilePayloadData(PayloadKey, AlternateTexturePath);
}

TOptional<UE::Interchange::FMeshPayloadData> UInterchangeDatasmithTranslator::GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const
{
	using namespace UE::DatasmithInterchange;
	using namespace UE::Interchange;
	FTransform MeshGlobalTransform;
	PayloadAttributes.GetAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);

	TOptional<UE::Interchange::FMeshPayloadData> EmptyPayload = TOptional<UE::Interchange::FMeshPayloadData>();

	if (!LoadedExternalSource || !LoadedExternalSource->GetDatasmithScene())
	{
		return EmptyPayload;
	}

	int32 MeshIndex = 0;
	LexFromString(MeshIndex, *PayLoadKey.UniqueId);
	TSharedPtr<IDatasmithScene> DatasmithScene = LoadedExternalSource->GetDatasmithScene();
	if (MeshIndex < 0 || MeshIndex >= DatasmithScene->GetMeshesCount())
	{
		return EmptyPayload;
	}

	TSharedPtr<IDatasmithMeshElement> MeshElement = DatasmithScene->GetMesh(MeshIndex);
	if (!MeshElement.IsValid())
	{
		return EmptyPayload;
	}

	UE::Interchange::FMeshPayloadData StaticMeshPayloadData;
	if (GetMeshDescription(MeshElement, MeshGlobalTransform, StaticMeshPayloadData))
	{
		TOptional<UE::Interchange::FMeshPayloadData> Payload;
		Payload = MoveTemp(StaticMeshPayloadData);
		return Payload;
	}

	return EmptyPayload;
}

TOptional<UE::Interchange::FAnimationPayloadData> UInterchangeDatasmithTranslator::GetAnimationPayloadData(const UE::Interchange::FAnimationPayloadQuery& PayloadQuery) const
{
	TOptional<UE::Interchange::FAnimationPayloadData> Result;

	if (!LoadedExternalSource || !LoadedExternalSource->GetDatasmithScene())
	{
		return Result;
	}

	TSharedPtr<IDatasmithBaseAnimationElement> AnimationElement;
	float FrameRate = 0.f;
	if (UE::DatasmithInterchange::AnimUtils::FAnimationPayloadDesc* PayloadDescPtr = AnimationPayLoadMapping.Find(PayloadQuery.PayloadKey.UniqueId))
	{
		AnimationElement = PayloadDescPtr->Value;
		if (!ensure(AnimationElement))
		{
			// #ueent_logwarning:
			return Result;
		}

		FrameRate = PayloadDescPtr->Key;
	}

	if (PayloadQuery.PayloadKey.Type != EInterchangeAnimationPayLoadType::NONE)
	{
		UE::Interchange::FAnimationPayloadData TransformPayloadData(PayloadQuery.SceneNodeUniqueID, PayloadQuery.PayloadKey);
		if (UE::DatasmithInterchange::AnimUtils::GetAnimationPayloadData(*AnimationElement, FrameRate, PayloadQuery.PayloadKey.Type, TransformPayloadData))
		{
			Result = MoveTemp(TransformPayloadData);
		}
	}

	return Result;
}

TArray<UE::Interchange::FAnimationPayloadData> UInterchangeDatasmithTranslator::GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const
{
	TArray<TOptional<UE::Interchange::FAnimationPayloadData>> AnimationPayloadOptionals;
	int32 PayloadCount = PayloadQueries.Num();
	AnimationPayloadOptionals.AddDefaulted(PayloadCount);

	const int32 BatchSize = 5;
	if (PayloadQueries.Num() > BatchSize)
	{
		const int32 NumBatches = (PayloadCount / BatchSize) + 1;
		ParallelFor(NumBatches, [&](int32 BatchIndex)
			{
				int32 PayloadIndexOffset = BatchIndex * BatchSize;
				for (int32 PayloadIndex = PayloadIndexOffset; PayloadIndex < PayloadIndexOffset + BatchSize; ++PayloadIndex)
				{
					if (PayloadQueries.IsValidIndex(PayloadIndex))
					{
						AnimationPayloadOptionals[PayloadIndex] = GetAnimationPayloadData(PayloadQueries[PayloadIndex]);
					}
				}
			}, EParallelForFlags::BackgroundPriority);// ParallelFor
	}
	else
	{
		for (int32 PayloadIndex = 0; PayloadIndex < PayloadCount; ++PayloadIndex)
		{
			if (PayloadQueries.IsValidIndex(PayloadIndex))
			{
				AnimationPayloadOptionals[PayloadIndex] = GetAnimationPayloadData(PayloadQueries[PayloadIndex]);
			}
		}
	}


	TArray<UE::Interchange::FAnimationPayloadData> AnimationPayloads;
	for (TOptional<UE::Interchange::FAnimationPayloadData>& OptionalPayloadData : AnimationPayloadOptionals)
	{
		if (!OptionalPayloadData.IsSet())
		{
			continue;
		}
		AnimationPayloads.Add(OptionalPayloadData.GetValue());
	}

	return AnimationPayloads;
}

TOptional<UE::Interchange::FVariantSetPayloadData> UInterchangeDatasmithTranslator::GetVariantSetPayloadData(const FString& PayloadKey) const
{
	using namespace UE::Interchange;
	using namespace UE::DatasmithInterchange;

	TOptional<FVariantSetPayloadData> Result;

	if (!LoadedExternalSource || !LoadedExternalSource->GetDatasmithScene())
	{
		return Result;
	}

	TSharedPtr<IDatasmithScene> DatasmithScene = LoadedExternalSource->GetDatasmithScene();
	
	TArray<FString> PayloadTokens;

	// We need two indices to build the payload: index of LevelVariantSet and index of VariantSetIndex
	if (2 != PayloadKey.ParseIntoArray(PayloadTokens, TEXT(";")))
	{
		// Invalid payload
		return Result;
	}

	int32 LevelVariantSetIndex = FCString::Atoi(*PayloadTokens[0]);
	int32 VariantSetIndex = FCString::Atoi(*PayloadTokens[1]);

	TSharedPtr<IDatasmithLevelVariantSetsElement> LevelVariantSetElement = DatasmithScene->GetLevelVariantSets(LevelVariantSetIndex);
	if (ensure(LevelVariantSetElement))
	{
		TSharedPtr<IDatasmithVariantSetElement> VariantSet = LevelVariantSetElement->GetVariantSet(VariantSetIndex);
		if (ensure(VariantSet) && VariantSet->GetVariantsCount() > 0)
		{
			TSharedPtr<TPromise<TOptional<FVariantSetPayloadData>>> Promise = MakeShared<TPromise<TOptional<FVariantSetPayloadData>>>();
			FVariantSetPayloadData PayloadData;
			if (VariantSetUtils::GetVariantSetPayloadData(*VariantSet, PayloadData))
			{
				Result = MoveTemp(PayloadData);
				return Result;
			}
		}
	}

	return Result;
}

void UInterchangeDatasmithTranslator::ImportFinish()
{
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int ElapsedMin = int(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;

	UE_LOGF(LogInterchangeDatasmith, Log, "Imported %ls in [%d min %.3f s]", *FileName, ElapsedMin, ElapsedSeconds);

	// Remove dependency on created static meshes
	StaticMeshDataNode->AdditionalDataMap.Reset();
	StaticMeshDataNode = nullptr;

	if (LoadedExternalSource.IsValid())
	{
		const TSharedPtr<IDatasmithTranslator>& DatasmithTranslator = LoadedExternalSource->GetAssetTranslator();
		if (DatasmithTranslator)
		{
			DatasmithTranslator->UnloadScene();
		}
	}

}


UInterchangeTranslatorSettings* UInterchangeDatasmithTranslator::GetSettings() const
{
	using namespace UE::DatasmithImporter;
	using namespace UE::DatasmithInterchange;

	if (!CachedSettings)
	{
		if (!LoadedExternalSource.IsValid())
		{
			FString FilePath = FPaths::ConvertRelativePathToFull(SourceData->GetFilename());
			FileName = FPaths::GetCleanFilename(FilePath);
			const FSourceUri FileNameUri = FSourceUri::FromFilePath(FilePath);
			LoadedExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FileNameUri);
		}

		if (!LoadedExternalSource.IsValid() || !LoadedExternalSource->IsAvailable())
		{
			return nullptr;
		}

		TArray<TObjectPtr<UDatasmithOptionsBase>> DatasmithOptions;
		const TSharedPtr<IDatasmithTranslator>& DatasmithTranslator = LoadedExternalSource->GetAssetTranslator();
		DatasmithTranslator->GetSceneImportOptions(DatasmithOptions);
		if (DatasmithOptions.Num() == 0)
		{
			return nullptr;
		}

		CachedSettings = DuplicateObject<UInterchangeDatasmithTranslatorSettings>(UInterchangeDatasmithTranslatorSettings::StaticClass()->GetDefaultObject<UInterchangeDatasmithTranslatorSettings>(), GetTransientPackage());
		CachedSettings->SetFlags(RF_Standalone);
		CachedSettings->ClearFlags(RF_ArchetypeObject);
		CachedSettings->ClearInternalFlags(EInternalObjectFlags::Async);

		// Only the first one is considered
		CachedSettings->DatasmithOption = DatasmithOptions[0];

		CachedSettings->DatasmithOption->LoadConfig();
	}

	return CachedSettings;
}

void UInterchangeDatasmithTranslator::SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings)
{
	using namespace UE::DatasmithImporter;

	if (CachedSettings)
	{
		CachedSettings->ClearFlags(RF_Standalone);
		CachedSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		CachedSettings = nullptr;
	}
	if (InterchangeTranslatorSettings)
	{
		CachedSettings = DuplicateObject<UInterchangeDatasmithTranslatorSettings>(Cast<UInterchangeDatasmithTranslatorSettings>(InterchangeTranslatorSettings), GetTransientPackage());
		CachedSettings->ClearInternalFlags(EInternalObjectFlags::Async);
		CachedSettings->SetFlags(RF_Standalone);

		CachedSettings->SaveConfig();

		CachedSettings->DatasmithOption->SaveConfig();

		if (!LoadedExternalSource.IsValid())
		{
			FString FilePath = FPaths::ConvertRelativePathToFull(SourceData->GetFilename());
			FileName = FPaths::GetCleanFilename(FilePath);
			const FSourceUri FileNameUri = FSourceUri::FromFilePath(FilePath);
			LoadedExternalSource = IExternalSourceModule::GetOrCreateExternalSource(FileNameUri);
		}

		if (LoadedExternalSource.IsValid() && LoadedExternalSource->IsAvailable())
		{
			const TSharedPtr<IDatasmithTranslator>& DatasmithTranslator = LoadedExternalSource->GetAssetTranslator();
			DatasmithTranslator->SetSceneImportOptions({ CachedSettings->DatasmithOption });
		}
	}
}

bool UInterchangeDatasmithTranslator::GetMeshDescription(const TSharedPtr<IDatasmithMeshElement>& MeshElement, const FTransform& MeshGlobalTransform, UE::Interchange::FMeshPayloadData& PayloadData) const
{
	using namespace UE::DatasmithInterchange;

	FDatasmithMeshElementPayload DatasmithMeshPayload;
	if (!LoadedExternalSource->GetAssetTranslator()->LoadStaticMesh(MeshElement.ToSharedRef(), DatasmithMeshPayload))
	{
		UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
		ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
		ErrorResult->Text = FText::Format(LOCTEXT("GetMeshPayloadData_LoadStaticMeshFail", "Failed to load mesh description for mesh element {0}."), FText::FromString(MeshElement->GetName()));
		return false;
	}

	if (DatasmithMeshPayload.LodMeshes.Num() > 0)
	{
#ifdef NO_MONOLITHIC_BUILD
		for (UDatasmithAdditionalData* AdditionalData : DatasmithMeshPayload.AdditionalData)
		{
			if (UDatasmithParametricSurfaceData* ParametricSurfaceData = Cast<UDatasmithParametricSurfaceData>(AdditionalData))
			{
				FWriteScopeLock ReconnectionScopeLock(StaticMeshDataNodeLock);

				const FString MeshNodeUid = NodeUtils::MeshPrefix + MeshElement->GetName();
				StaticMeshDataNode->AdditionalDataMap.Add(MeshNodeUid, ParametricSurfaceData);
				break;
			}
		}
#endif

		if (!FStaticMeshOperations::ValidateAndFixData(DatasmithMeshPayload.LodMeshes[0], MeshElement->GetName()))
		{
			UInterchangeResultError_Generic* ErrorResult = AddMessage<UInterchangeResultError_Generic>();
			ErrorResult->SourceAssetName = SourceData ? SourceData->GetFilename() : FString();
			ErrorResult->Text = FText::Format(LOCTEXT("GetMeshPayloadData_ValidateMeshDescriptionFail", "Invalid mesh data (NAN) was found and fix to zero. Mesh render can be bad for mesh element {0}."), FText::FromString(MeshElement->GetName()));
			return false;
		}
		// Bake the payload mesh, with the provided transform
		if (!MeshGlobalTransform.Equals(FTransform::Identity))
		{
			FStaticMeshOperations::ApplyTransform(DatasmithMeshPayload.LodMeshes[0], MeshGlobalTransform);
		}

		PayloadData.MeshDescription = MoveTemp(DatasmithMeshPayload.LodMeshes[0]);
	}

	return true;
}

#undef LOCTEXT_NAMESPACE