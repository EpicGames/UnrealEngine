// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/InterchangeAnimationPayloadInterface.h"
#include "InterchangeCommonAnimationPayload.h"
#include "InterchangeTranslatorBase.h"
#include "Mesh/InterchangeMeshPayloadInterface.h"
#include "Texture/InterchangeTexturePayloadData.h"
#include "Texture/InterchangeTexturePayloadInterface.h"
#include "Texture/InterchangeTextureLightProfilePayloadData.h"
#include "Texture/InterchangeTextureLightProfilePayloadInterface.h"
#include "Scene/InterchangeVariantSetPayloadInterface.h"

#include "DatasmithImportOptions.h"

#include "Async/Async.h"
#include "ExternalSource.h"
#include "UObject/GCObjectScopeGuard.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeDatasmithTranslator.generated.h"

class IDatasmithActorElement;
class IDatasmithBaseAnimationElement;
class IDatasmithCameraActorElement;
class IDatasmithLightActorElement;
class IDatasmithDecalActorElement;
class IDatasmithMeshElement;
class IDatasmithScene;
class IDatasmithTransformAnimationElement;
class UDatasmithOptionsBase;
class UDatasmithInterchangeStaticMeshDataNode;
class UInterchangePhysicalCameraNode;
class UInterchangeBaseLightNode;
class UInterchangeDecalNode;
class UInterchangeLightNode;
class UInterchangeSceneNode;

namespace UE::Interchange
{
	struct FImportImage;
	struct FMeshPayloadData;
	struct FAnimationTransformPayloadData;
	struct FAnimationBakeTransformPayloadData;
}

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

namespace UE::DatasmithInterchange::AnimUtils
{
	typedef TPair<float, TSharedPtr<IDatasmithBaseAnimationElement>> FAnimationPayloadDesc;
	extern bool GetAnimationPayloadData(const IDatasmithBaseAnimationElement& AnimationElement, float FrameRate, EInterchangeAnimationPayLoadType PayLoadType, UE::Interchange::FAnimationPayloadData& PayLoadData);
}

UENUM()
enum class EInterchangeMesherType : uint8
{
	UseCADKernel UMETA(DisplayName = "Use CADKernel"),
	UseTechSoft,
	UseNativeTessellator,
};

UCLASS(BlueprintType, editinlinenew, MinimalAPI)
class UInterchangeDatasmithTranslatorSettings : public UInterchangeTranslatorSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Datasmith Options")
	TObjectPtr<UDatasmithOptionsBase> DatasmithOption;
};

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithTranslator : public UInterchangeTranslatorBase
	, public IInterchangeTexturePayloadInterface
	, public IInterchangeTextureLightProfilePayloadInterface
	, public IInterchangeMeshPayloadInterface
	, public IInterchangeAnimationPayloadInterface
	, public IInterchangeVariantSetPayloadInterface
{
	GENERATED_BODY()

public:

	/** Begin UInterchangeTranslatorBase API*/
	virtual bool CanImportSourceData(const UInterchangeSourceData* InSourceData) const override;

	virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const override;

	virtual TArray<FString> GetSupportedFormats() const override;
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const override
	{ 
		return EInterchangeTranslatorAssetType::Textures | EInterchangeTranslatorAssetType::Materials | EInterchangeTranslatorAssetType::Meshes;
	}
	virtual EInterchangeTranslatorType GetTranslatorType() const override
	{
		return EInterchangeTranslatorType::Scenes;
	}

	virtual void ReleaseSource() override
	{
		return;
	}

	virtual void ImportFinish() override;

	virtual UInterchangeTranslatorSettings* GetSettings() const override;
	virtual void SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings) override;
	/** End UInterchangeTranslatorBase API*/

	/* IInterchangeTexturePayloadInterface Begin */
	virtual TOptional<UE::Interchange::FImportImage> GetTexturePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	/* IInterchangeTexturePayloadInterface End */

	/* IInterchangeTextureLightProfilePayloadInterface Begin */
	virtual TOptional<UE::Interchange::FImportLightProfile> GetLightProfilePayloadData(const FString& PayloadKey, TOptional<FString>& AlternateTexturePath) const override;
	/* IInterchangeTextureLightProfilePayloadInterface End */

	/* IInterchangeStaticMeshPayloadInterface Begin */
	UE_DEPRECATED(5.6, "Deprecated. Use GetMeshPayloadData(const FInterchangeMeshPayLoadKey&) instead.")
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const FTransform& MeshGlobalTransform) const override
	{
		using namespace UE::Interchange;
		FAttributeStorage Attributes;
		Attributes.RegisterAttribute(UE::Interchange::FAttributeKey{ MeshPayload::Attributes::MeshGlobalTransform }, MeshGlobalTransform);
		return GetMeshPayloadData(PayLoadKey, Attributes);
	}
	virtual TOptional<UE::Interchange::FMeshPayloadData> GetMeshPayloadData(const FInterchangeMeshPayLoadKey& PayLoadKey, const UE::Interchange::FAttributeStorage& PayloadAttributes) const override;
	/* IInterchangeStaticMeshPayloadInterface End */

	/* IInterchangeAnimationPayloadInterface Begin */
	TOptional<UE::Interchange::FAnimationPayloadData> GetAnimationPayloadData(const UE::Interchange::FAnimationPayloadQuery& PayloadQuery) const;
	virtual TArray<UE::Interchange::FAnimationPayloadData> GetAnimationPayloadData(const TArray<UE::Interchange::FAnimationPayloadQuery>& PayloadQueries) const override;
	/* IInterchangeAnimationPayloadInterface End */

	/* IInterchangeVariantSetPayloadInterface Begin */
	virtual TOptional<UE::Interchange::FVariantSetPayloadData> GetVariantSetPayloadData(const FString& PayloadKey) const override;
	/* IInterchangeVariantSetPayloadInterface End */

private:

	void HandleDatasmithActor(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithActorElement>& ActorElement, const UInterchangeSceneNode* ParentNode) const;

	UInterchangePhysicalCameraNode* AddCameraNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithCameraActorElement>& CameraActor) const;

	UInterchangeBaseLightNode* AddLightNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithLightActorElement>& LightActor) const;

	UInterchangeDecalNode* AddDecalNode(UInterchangeBaseNodeContainer& BaseNodeContainer, const TSharedRef<IDatasmithDecalActorElement>& DecalActor) const;

	void ProcessIesProfile(UInterchangeBaseNodeContainer& BaseNodeContainer, const IDatasmithLightActorElement& LightElement, UInterchangeLightNode* LightNode) const;

	bool GetMeshDescription(const TSharedPtr<IDatasmithMeshElement>& MeshElement, const FTransform& MeshGlobalTransform, UE::Interchange::FMeshPayloadData& PayloadData) const;

	mutable TSharedPtr<UE::DatasmithImporter::FExternalSource> LoadedExternalSource;

	mutable uint64 StartTime = 0;
	mutable FString FileName;

	UPROPERTY(Transient, DuplicateTransient)
	mutable TObjectPtr<UInterchangeDatasmithTranslatorSettings> CachedSettings = nullptr;

	mutable TMap<FString, UE::DatasmithInterchange::AnimUtils::FAnimationPayloadDesc> AnimationPayLoadMapping;

	static FRWLock StaticMeshDataNodeLock;
	mutable TObjectPtr<UDatasmithInterchangeStaticMeshDataNode> StaticMeshDataNode;
	mutable EAsyncExecution AsyncMode = EAsyncExecution::TaskGraph;
};
