// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeDatasmithUtils.h"

#include "InterchangeVariantSetNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Scene/InterchangeVariantSetPayloadInterface.h"

#include "DatasmithVariantElements.h"

#include "Templates/SharedPointer.h"

namespace UE::DatasmithInterchange::NodeUtils
{
	const FString ActorPrefix(FString(TEXT("\\Actor\\")));
	const FString CameraPrefix(FString(TEXT("\\Camera\\")));
	const FString DatasmithScenePrefix(FString(TEXT("\\DatasmithScene\\")));
	const FString LevelSequencePrefix(FString(TEXT("\\LevelSequence\\")));
	const FString LevelVariantSetPrefix(FString(TEXT("\\LevelVariantSet\\")));
	const FString LightPrefix(FString(TEXT("\\Light\\")));
	const FString DecalPrefix(FString(TEXT("\\Decal\\")));
	const FString DecalMaterialPrefix(FString(TEXT("\\DecalMaterial\\")));
	const FString MaterialPrefix(FString(TEXT("\\Material\\")));
	const FString MaterialExpressionPrefix(FString(TEXT("\\MaterialExpression\\")));
	const FString MaterialFunctionPrefix(FString(TEXT("\\MaterialFunction\\")));
	const FString MeshPrefix(FString(TEXT("\\StaticMesh\\")));
	const FString ScenePrefix(FString(TEXT("\\Scene\\")));
	const FString TexturePrefix(FString(TEXT("\\Texture\\")));
	const FString VariantSetPrefix(FString(TEXT("\\VariantSet\\")));

	UInterchangeFactoryBaseNode* FindFactoryNodeFromAsset(const UInterchangeBaseNodeContainer* NodeContainer, const UObject* Asset)
	{
		UInterchangeFactoryBaseNode* FactoryNode = nullptr;
		NodeContainer->BreakableIterateNodesOfType<UInterchangeFactoryBaseNode>([&Asset, &FactoryNode](const FString& NodeUid, UInterchangeFactoryBaseNode* Node)
			{
				FSoftObjectPath NodeReferenceObject;
				Node->GetCustomReferenceObject(NodeReferenceObject);
				if (NodeReferenceObject.TryLoad() == Asset)
				{
					FactoryNode = Node;
					return true;
				}
				return false;
			});

		return FactoryNode;
	}

	FString GetActorUid(const TCHAR* Name)
	{
		return ActorPrefix + Name;
	}

	FString GetLevelSequenceUid(const TCHAR* Name)
	{
		return LevelSequencePrefix + Name;
	}

	FString GetLevelVariantSetUid(const TCHAR* Name)
	{
		return LevelVariantSetPrefix + Name;
	}

	FString GetMaterialUid(const TCHAR* Name)
	{
		return MaterialPrefix + Name;
	}

	FString GetVariantSetUid(const TCHAR* Name)
	{
		return VariantSetPrefix + Name;
	}
}

namespace UE::DatasmithInterchange::MeshUtils
{
	const FString MeshMaterialAttrName(TEXT("Datasmith:Mesh:MaterialOverride"));
}

namespace UE::DatasmithInterchange::VariantSetUtils
{
	void TranslateLevelVariantSets(const TArray<TSharedPtr<IDatasmithLevelVariantSetsElement>>& LevelVariantSets, UInterchangeBaseNodeContainer& BaseNodeContainer)
	{
		int32 LevelVariantSetIndex = -1;
		for (const TSharedPtr<IDatasmithLevelVariantSetsElement>& LevelVariantSet : LevelVariantSets)
		{
			++LevelVariantSetIndex;

			if (!LevelVariantSet || LevelVariantSet->GetVariantSetsCount() == 0)
			{
				continue;
			}

			UInterchangeSceneVariantSetsNode* LevelVariantSetNode = NewObject< UInterchangeSceneVariantSetsNode >(&BaseNodeContainer);
			if (!ensure(LevelVariantSetNode))
			{
				continue;
			}

			const FString LevelVariantSetNodeUid = NodeUtils::GetLevelVariantSetUid(LevelVariantSet->GetName());
			//For LevelVariantSetNode it should only be added to the NodeContainer and used if at least 1 Depndency on it have been added.
			LevelVariantSetNode->InitializeNode(LevelVariantSetNodeUid, LevelVariantSet->GetLabel(), EInterchangeNodeContainerType::TranslatedScene);

			for (int32 VariantSetIndex = 0; VariantSetIndex < LevelVariantSet->GetVariantSetsCount(); ++VariantSetIndex)
			{
				TSharedPtr<IDatasmithVariantSetElement> VariantSet = LevelVariantSet->GetVariantSet(VariantSetIndex);
				if (!VariantSet || VariantSet->GetVariantsCount() == 0)
				{
					continue;
				}

				UInterchangeVariantSetNode* VariantSetNode = NewObject< UInterchangeVariantSetNode >(&BaseNodeContainer);
				if (!ensure(VariantSetNode))
				{
					continue;
				}

				const FString VariantSetNodeUid = NodeUtils::GetVariantSetUid(VariantSet->GetName());
				//For VariantSetNode it should only be added to the NodeContainer and used if at least 1 Depndency on it have been added.
				VariantSetNode->InitializeNode(VariantSetNodeUid, VariantSet->GetLabel(), EInterchangeNodeContainerType::TranslatedScene);

				for (int32 VariantIndex = 0; VariantIndex < VariantSet->GetVariantsCount(); ++VariantIndex)
				{
					if (TSharedPtr<IDatasmithVariantElement> Variant = VariantSet->GetVariant(VariantIndex))
					{
						for (int32 BindingIndex = 0; BindingIndex < Variant->GetActorBindingsCount(); ++BindingIndex)
						{
							TSharedPtr<IDatasmithActorBindingElement> Binding = Variant->GetActorBinding(BindingIndex);
							if (!Binding || Binding->GetPropertyCapturesCount() == 0)
							{
								continue;
							}

							if (TSharedPtr<IDatasmithActorElement> Actor = Binding->GetActor())
							{
								bool bMustImport = false;

								for (int32 PropertyCapturIndex = 0; PropertyCapturIndex < Binding->GetPropertyCapturesCount(); ++PropertyCapturIndex)
								{
									if (TSharedPtr<IDatasmithBasePropertyCaptureElement> BasePropertyCapture = Binding->GetPropertyCapture(PropertyCapturIndex))
									{
										// If property capture is an object, add its corresponding unique id as a dependency
										if (BasePropertyCapture->IsSubType(EDatasmithElementVariantSubType::ObjectPropertyCapture))
										{
											TSharedPtr<IDatasmithObjectPropertyCaptureElement> PropertyCapture = StaticCastSharedRef<IDatasmithObjectPropertyCaptureElement>(BasePropertyCapture.ToSharedRef());

											if (TSharedPtr<IDatasmithElement> TargetElement = PropertyCapture->GetRecordedObject().Pin())
											{
												if (TargetElement->IsA(EDatasmithElementType::BaseMaterial))
												{
													VariantSetNode->AddCustomDependencyUid(NodeUtils::GetMaterialUid(TargetElement->GetName()));
												}
											}
										}

										bMustImport = true;
									}
								}

								if (bMustImport)
								{
									VariantSetNode->AddCustomDependencyUid(NodeUtils::GetActorUid(Actor->GetName()));
								}
							}
						}
					}
				}

				if (VariantSetNode->GetCustomDependencyUidCount() > 0)
				{
					VariantSetNode->SetCustomVariantsPayloadKey(FString::Printf(TEXT("%d;%d"), LevelVariantSetIndex, VariantSetIndex));

					VariantSetNode->SetCustomDisplayText(VariantSet->GetName());

					LevelVariantSetNode->AddCustomVariantSetUid(VariantSetNodeUid);
					BaseNodeContainer.AddNode(VariantSetNode);
				}
			}

			// Only add to container if there is something to import
			if (LevelVariantSetNode->GetCustomVariantSetUidCount() > 0)
			{
				BaseNodeContainer.AddNode(LevelVariantSetNode);
			}
		}
	}

	bool GetVariantSetPayloadData(const IDatasmithVariantSetElement& VariantSet, UE::Interchange::FVariantSetPayloadData& PayLoadData)
	{
		using namespace UE::Interchange;

		PayLoadData.Variants.Reserve(VariantSet.GetVariantsCount());

		for (int32 VariantIndex = 0; VariantIndex < VariantSet.GetVariantsCount(); ++VariantIndex)
		{
			// Temporarily using const_cast since IDatasmithVariantSetElement::GetVariant does not have (yet) a const alternative.
			if (const TSharedPtr<IDatasmithVariantElement> VariantElement = const_cast<IDatasmithVariantSetElement&>(VariantSet).GetVariant(VariantIndex))
			{
				TArray<FVariantBinding> Bindings;
				Bindings.Reserve(VariantElement->GetActorBindingsCount());

				for (int32 BindingIndex = 0; BindingIndex < VariantElement->GetActorBindingsCount(); ++BindingIndex)
				{
					TSharedPtr<IDatasmithActorBindingElement> BindingElement = VariantElement->GetActorBinding(BindingIndex);
					if (!BindingElement || BindingElement->GetPropertyCapturesCount() == 0)
					{
						continue;
					}

					if (TSharedPtr<IDatasmithActorElement> Actor = BindingElement->GetActor())
					{
						TArray<FVariantPropertyCaptureData> Captures;
						Captures.Reserve(BindingElement->GetPropertyCapturesCount());

						for (int32 PropertyCapturIndex = 0; PropertyCapturIndex < BindingElement->GetPropertyCapturesCount(); ++PropertyCapturIndex)
						{
							if (TSharedPtr<IDatasmithBasePropertyCaptureElement> BasePropertyCapture = BindingElement->GetPropertyCapture(PropertyCapturIndex))
							{
								FVariantPropertyCaptureData& PropertyCaptureData = Captures.AddDefaulted_GetRef();

								PropertyCaptureData.Category = (Interchange::EVariantPropertyCaptureCategory)BasePropertyCapture->GetCategory();
								PropertyCaptureData.PropertyPath = BasePropertyCapture->GetPropertyPath();

								// If property capture is an object, add its corresponding unique id as a dependency
								if (BasePropertyCapture->IsSubType(EDatasmithElementVariantSubType::ObjectPropertyCapture))
								{
									TSharedPtr<IDatasmithObjectPropertyCaptureElement> PropertyCapture = StaticCastSharedRef<IDatasmithObjectPropertyCaptureElement>(BasePropertyCapture.ToSharedRef());

									if (TSharedPtr<IDatasmithElement> TargetElement = PropertyCapture->GetRecordedObject().Pin())
									{
										if (TargetElement->IsA(EDatasmithElementType::BaseMaterial))
										{
											PropertyCaptureData.ObjectUid = NodeUtils::GetMaterialUid(TargetElement->GetName());
										}
									}
								}
								else
								{
									TSharedPtr<IDatasmithPropertyCaptureElement> PropertyCapture = StaticCastSharedRef<IDatasmithPropertyCaptureElement>(BasePropertyCapture.ToSharedRef());
									PropertyCaptureData.Buffer = PropertyCapture->GetRecordedData();
								}
							}
						}

						if (Captures.Num() > 0)
						{
							FVariantBinding& Binding = Bindings.AddDefaulted_GetRef();

							Binding.TargetUid = NodeUtils::GetActorUid(Actor->GetName());
							Binding.Captures = MoveTemp(Captures);
						}
					}
				}

				if (Bindings.Num() > 0)
				{
					FVariant& Variant = PayLoadData.Variants.AddDefaulted_GetRef();

					Variant.DisplayText = VariantElement->GetName();
					Variant.Bindings = MoveTemp(Bindings);
				}
			}
		}

		return PayLoadData.Variants.Num() > 0;
	}
}
