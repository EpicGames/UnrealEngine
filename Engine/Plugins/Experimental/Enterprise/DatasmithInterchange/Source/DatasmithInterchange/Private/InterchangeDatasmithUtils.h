// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "IDatasmithSceneElements.h"

class IDatasmithBaseAnimationElement;
class IDatasmithBaseMaterialElement;
class IDatasmithLevelVariantSetsElement;
class IDatasmithVariantSetElement;

namespace UE::Interchange
{
	struct FAnimationCurvePayloadData;
	struct FAnimationStepCurvePayloadData;
	struct FVariantSetPayloadData;
}

namespace UE::DatasmithInterchange::NodeUtils
{
	extern const FString ActorPrefix;
	extern const FString CameraPrefix;
	extern const FString DatasmithScenePrefix;
	extern const FString LevelSequencePrefix;
	extern const FString LevelVariantSetPrefix;
	extern const FString LightPrefix;
	extern const FString DecalPrefix;
	extern const FString DecalMaterialPrefix;
	extern const FString MaterialPrefix;
	extern const FString MaterialExpressionPrefix;
	extern const FString MaterialFunctionPrefix;
	extern const FString MeshPrefix;
	extern const FString ScenePrefix;
	extern const FString TexturePrefix;
	extern const FString VariantSetPrefix;

	extern FString GetActorUid(const TCHAR* Name);
	extern FString GetLevelSequenceUid(const TCHAR* Name);
	extern FString GetLevelVariantSetUid(const TCHAR* Name);
	extern FString GetMaterialUid(const TCHAR* Name);
	extern FString GetVariantSetUid(const TCHAR* Name);

	// TODO: This is just a quick helper to get the code started, the real solution would be to use caching and pre-parse the NodeContainer to avoid O(n^2) query.
	template<typename T>
	TArray<T*> GetNodes(const UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
		TArray<T*> TypedNodes;

		BaseNodeContainer->GetNodesOfType<T>(TypedNodes);

		return TypedNodes;
	}

	template<class T>
	T* FindOrAddFactoryNode(const UInterchangeBaseNode* Node, UInterchangeBaseNodeContainer* NodeContainer, const FString& FactoryNodeUid)
	{
		T* FactoryNode = nullptr;

		if (NodeContainer->IsNodeUidValid(FactoryNodeUid))
		{
			//The node already exist, just return it
			FactoryNode = Cast<T>(NodeContainer->GetFactoryNode(FactoryNodeUid));
			if (!ensure(FactoryNode))
			{
				//Log an error
			}
		}
		else
		{
			FactoryNode = NewObject<T>(NodeContainer, T::StaticClass(), NAME_None);
			if (!ensure(FactoryNode))
			{
				return nullptr;
			}

			//Creating a Material
			NodeContainer->SetupNode(FactoryNode, FactoryNodeUid, Node->GetDisplayLabel(), EInterchangeNodeContainerType::FactoryData);
			
			FactoryNode->AddTargetNodeUid(Node->GetUniqueID());
			FactoryNode->SetAssetName(Node->GetAssetName());

			Node->AddTargetNodeUid(FactoryNode->GetUniqueID());
		}

		return FactoryNode;
	}

	UInterchangeFactoryBaseNode* FindFactoryNodeFromAsset(const UInterchangeBaseNodeContainer* NodeContainer, const UObject* Asset);

	template<class T = UInterchangeFactoryBaseNode>
	T* FindFactoryNodeByUniqueID(UInterchangeBaseNodeContainer* NodeContainer, const FString& NodeUniqueID)
	{
		T* FactoryNode = nullptr;
		NodeContainer->BreakableIterateNodes([&NodeUniqueID, &FactoryNode](const FString& NodeUid, UInterchangeBaseNode* Node)
			{
				if (NodeUniqueID == NodeUid)
				{
					FactoryNode = Cast<T>(Node);
					return true;
				}
				return false;
			});

		ensure(FactoryNode);

		return FactoryNode;
	}


	template<typename DatasmithElementType>
	int32 GetDatasmithElementsCount(const TSharedRef<IDatasmithScene>& DatasmithScene); //no definition, template should be specialized

	template<typename DatasmithElementType>
	TSharedPtr<DatasmithElementType> GetDatasmithElement(const TSharedRef<IDatasmithScene>& DatasmithScene, int32 ElementIndex); //no definition, template should be specialized

	template<typename DatasmithElementType>
	TSharedPtr<DatasmithElementType> GetDatasmithElementFromIndex(const TSharedRef<IDatasmithScene>& DatasmithScene, int32 ElementIndex)
	{
		if (ElementIndex < 0 || ElementIndex >= GetDatasmithElementsCount<DatasmithElementType>(DatasmithScene))
		{
			return TSharedPtr<DatasmithElementType>();
		}

		return GetDatasmithElement<DatasmithElementType>(DatasmithScene, ElementIndex);
	}

	template<typename DatasmithElementType>
	TSharedPtr<DatasmithElementType> GetDatasmithElementFromPayloadKey(const TSharedRef<IDatasmithScene>& DatasmithScene, const FString& PayloadKey)
	{
		int32 ElementIndex = 0;
		LexFromString(ElementIndex, *PayloadKey);

		return GetDatasmithElementFromIndex<DatasmithElementType>(DatasmithScene, ElementIndex);
	}
}

namespace UE::DatasmithInterchange::MeshUtils
{
	extern const FString MeshMaterialAttrName;
}

namespace UE::DatasmithInterchange::AnimUtils
{

	typedef TPair<float, TSharedPtr<IDatasmithBaseAnimationElement>> FAnimationPayloadDesc;

	extern void TranslateLevelSequences(TArray<TSharedPtr<IDatasmithLevelSequenceElement>>& LevelSequences, UInterchangeBaseNodeContainer& InBaseNodeContainer, TMap<FString, UE::DatasmithInterchange::AnimUtils::FAnimationPayloadDesc>& AnimationPayLoadMapping);
	extern bool GetAnimationPayloadData(const IDatasmithBaseAnimationElement& AnimationElement, float FrameRate, UE::Interchange::FAnimationCurvePayloadData& PayLoadData);
	extern bool GetAnimationPayloadData(const IDatasmithBaseAnimationElement& AnimationElement, float FrameRate, UE::Interchange::FAnimationStepCurvePayloadData& PayLoadData);
}

namespace UE::DatasmithInterchange::VariantSetUtils
{

	extern void TranslateLevelVariantSets(const TArray<TSharedPtr<IDatasmithLevelVariantSetsElement>>& LevelVariantSets, UInterchangeBaseNodeContainer& InBaseNodeContainer);
	extern bool GetVariantSetPayloadData(const IDatasmithVariantSetElement& VariantSet, UE::Interchange::FVariantSetPayloadData& PayLoadData);
}

template<>
inline int32 UE::DatasmithInterchange::NodeUtils::GetDatasmithElementsCount<IDatasmithTextureElement>(const TSharedRef<IDatasmithScene>& DatasmithScene) { return DatasmithScene->GetTexturesCount(); }
template<>
inline int32 UE::DatasmithInterchange::NodeUtils::GetDatasmithElementsCount<IDatasmithBaseMaterialElement>(const TSharedRef<IDatasmithScene>& DatasmithScene) { return DatasmithScene->GetMaterialsCount(); }
template<>
inline int32 UE::DatasmithInterchange::NodeUtils::GetDatasmithElementsCount<IDatasmithMeshElement>(const TSharedRef<IDatasmithScene>& DatasmithScene) { return DatasmithScene->GetMeshesCount(); }

template<>
inline TSharedPtr<IDatasmithTextureElement> UE::DatasmithInterchange::NodeUtils::GetDatasmithElement<IDatasmithTextureElement>(const TSharedRef<IDatasmithScene>& DatasmithScene, int32 ElementIndex) { return DatasmithScene->GetTexture(ElementIndex); }
template<>
inline TSharedPtr<IDatasmithBaseMaterialElement> UE::DatasmithInterchange::NodeUtils::GetDatasmithElement<IDatasmithBaseMaterialElement>(const TSharedRef<IDatasmithScene>& DatasmithScene, int32 ElementIndex) { return DatasmithScene->GetMaterial(ElementIndex); }
template<>
inline TSharedPtr<IDatasmithMeshElement> UE::DatasmithInterchange::NodeUtils::GetDatasmithElement<IDatasmithMeshElement>(const TSharedRef<IDatasmithScene>& DatasmithScene, int32 ElementIndex) { return DatasmithScene->GetMesh(ElementIndex); }