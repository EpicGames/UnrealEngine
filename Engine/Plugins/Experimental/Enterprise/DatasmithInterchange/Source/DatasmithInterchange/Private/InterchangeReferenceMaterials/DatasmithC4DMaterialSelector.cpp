// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithC4DMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		namespace C4D
		{
			const TCHAR* MaterialPath = TEXT("/DatasmithContent/Materials/C4DReference.C4DReference");
		}

		FDatasmithC4DMaterialSelector::FDatasmithC4DMaterialSelector()
		{
			bIsValid = FPackageName::DoesPackageExist(C4D::MaterialPath);
		}

		const TCHAR* FDatasmithC4DMaterialSelector::GetMaterialPath(EDatasmithReferenceMaterialType) const
		{
			return C4D::MaterialPath;
		}

#if WITH_EDITOR
		void FDatasmithC4DMaterialSelector::PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality, UMaterialInstanceConstant* MaterialInstance) const
		{
			if (MaterialInstance)
			{
				// Set blend mode to translucent if material requires transparency.
				if (MaterialType == EDatasmithReferenceMaterialType::Transparent)
				{
					MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
					MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Translucent;
				}
				// Set blend mode to masked if material has cutouts.
				else if (MaterialType == EDatasmithReferenceMaterialType::CutOut)
				{
					MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
					MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Masked;
				}
			}
		}
#endif
	}
}