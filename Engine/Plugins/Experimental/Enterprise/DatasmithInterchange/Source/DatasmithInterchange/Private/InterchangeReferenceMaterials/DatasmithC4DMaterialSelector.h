// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithDefinitions.h"
#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "Templates/SharedPointer.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		class FDatasmithC4DMaterialSelector : public FDatasmithReferenceMaterialSelector
		{
		public:

			FDatasmithC4DMaterialSelector();
			virtual ~FDatasmithC4DMaterialSelector() = default;

			virtual const TCHAR* GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const override;
#if WITH_EDITOR
			virtual void PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality MaterialQuality, UMaterialInstanceConstant* MaterialInstance) const override;
#endif
		};
	}
}
