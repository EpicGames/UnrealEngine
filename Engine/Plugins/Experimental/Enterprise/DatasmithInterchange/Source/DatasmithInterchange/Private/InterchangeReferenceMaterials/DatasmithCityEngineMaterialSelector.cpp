// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithCityEngineMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Casts.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		namespace CityEngine
		{
			const TCHAR* MaterialPath = TEXT("/DatasmithContent/Materials/CE_OpaqueReference.CE_OpaqueReference");
			const TCHAR* MaterialTransparentPath = TEXT("/DatasmithContent/Materials/CE_OpacityReference.CE_OpacityReference");
			const TCHAR* MaterialTransparentSimplePath = TEXT("/DatasmithContent/Materials/CE_OpacitySimpleReference.CE_OpacitySimpleReference");
		}

		FDatasmithCityEngineMaterialSelector::FDatasmithCityEngineMaterialSelector()
		{
			bIsValid = FPackageName::DoesPackageExist(CityEngine::MaterialPath) || FPackageName::DoesPackageExist(CityEngine::MaterialTransparentPath) || FPackageName::DoesPackageExist(CityEngine::MaterialTransparentSimplePath);
		}

		const TCHAR* FDatasmithCityEngineMaterialSelector::GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const
		{
			switch (MaterialType)
			{
			case EDatasmithReferenceMaterialType::Transparent:
				return CityEngine::MaterialTransparentPath;
			default:
				return CityEngine::MaterialPath;
			}
		}

#if WITH_EDITOR
		void FDatasmithCityEngineMaterialSelector::PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality MaterialQuality, UMaterialInstanceConstant* MaterialInstance) const
		{
			if (MaterialInstance == nullptr || MaterialInstance->Parent == nullptr)
			{
				return;
			}

			if (MaterialInstance->Parent->GetFullName().Contains(CityEngine::MaterialTransparentPath))
			{
				if (MaterialQuality == EDatasmithReferenceMaterialQuality::Low)
				{
					MaterialInstance->Parent = Cast<UMaterialInterface>(FSoftObjectPath(CityEngine::MaterialTransparentSimplePath).TryLoad());
					if (!ensure(MaterialInstance->Parent))
					{
						MaterialInstance->Parent = Cast<UMaterialInterface>(FSoftObjectPath(CityEngine::MaterialTransparentPath).TryLoad());
					}
				}
			}
			else
			{
				float OpacityValue = 1.f;
				bool bIsTransparent = MaterialInstance->GetScalarParameterValue(TEXT("Opacity"), OpacityValue) && OpacityValue < 1.f;

				class UTexture* TextureValue = nullptr;
				MaterialInstance->GetTextureParameterValue(TEXT("OpacityMap"), TextureValue);
				bIsTransparent |= TextureValue != nullptr;

				if (bIsTransparent)
				{
					UMaterialInterface* PrevParent = MaterialInstance->Parent;

					if (MaterialQuality == EDatasmithReferenceMaterialQuality::Low)
					{
						MaterialInstance->Parent = Cast<UMaterialInterface>(FSoftObjectPath(CityEngine::MaterialTransparentSimplePath).TryLoad());
					}
					else
					{
						MaterialInstance->Parent = Cast<UMaterialInterface>(FSoftObjectPath(CityEngine::MaterialTransparentPath).TryLoad());
					}

					if (!ensure(MaterialInstance->Parent))
					{
						MaterialInstance->Parent = PrevParent;
					}
				}
			}
		}
#endif
	}
}