// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRevitMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"
//#include "UObject/SoftObjectPath.h"
//#include "Templates/Casts.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		namespace Revit
		{
			const TCHAR* MaterialPath = TEXT("/DatasmithContent/Materials/RevitReference.RevitReference");
		}

		FDatasmithRevitMaterialSelector::FDatasmithRevitMaterialSelector()
		{
			bIsValid = FPackageName::DoesPackageExist(Revit::MaterialPath);
		}

		const TCHAR* FDatasmithRevitMaterialSelector::GetMaterialPath(EDatasmithReferenceMaterialType) const
		{
			return Revit::MaterialPath;
		}

#if WITH_EDITOR
		void FDatasmithRevitMaterialSelector::PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality, UMaterialInstanceConstant* MaterialInstance) const
		{
			if (MaterialInstance)
			{
				// Set blend mode to translucent if material requires transparency
				if (MaterialType == EDatasmithReferenceMaterialType::Transparent)
				{
					MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
					MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Translucent;
				}
				// Set blend mode to masked if material has cutouts
				else if (MaterialType == EDatasmithReferenceMaterialType::CutOut)
				{
					MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
					MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Masked;
				}
			}
		}
#endif
	}
}