// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"
#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "Templates/SharedPointer.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		class FDatasmithRevitMaterialSelector : public FDatasmithReferenceMaterialSelector
		{
		public:
			FDatasmithRevitMaterialSelector();
			virtual ~FDatasmithRevitMaterialSelector() = default;

			virtual const TCHAR* GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const override;
#if WITH_EDITOR
			virtual void PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality MaterialQuality, UMaterialInstanceConstant* MaterialInstance) const override;
#endif
		};
	}
}
