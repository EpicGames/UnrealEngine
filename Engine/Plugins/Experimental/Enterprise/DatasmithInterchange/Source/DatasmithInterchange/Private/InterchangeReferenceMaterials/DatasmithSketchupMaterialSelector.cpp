// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithSketchupMaterialSelector.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Misc/PackageName.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		namespace SketchUp
		{
			const TCHAR* MaterialPath = TEXT("/DatasmithContent/Materials/SketchupReference.SketchupReference");
		}

		FDatasmithSketchUpMaterialSelector::FDatasmithSketchUpMaterialSelector()
		{
			bIsValid = FPackageName::DoesPackageExist(SketchUp::MaterialPath);
		}

		const TCHAR* FDatasmithSketchUpMaterialSelector::GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const
		{
			return SketchUp::MaterialPath;
		}

#if WITH_EDITOR
		void FDatasmithSketchUpMaterialSelector::PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality, UMaterialInstanceConstant* MaterialInstance) const
		{
			if (MaterialInstance)
			{
				if (MaterialType == EDatasmithReferenceMaterialType::Transparent)
				{
					MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
					MaterialInstance->BasePropertyOverrides.BlendMode = BLEND_Translucent;
				}
			}

		}
#endif
	}
}