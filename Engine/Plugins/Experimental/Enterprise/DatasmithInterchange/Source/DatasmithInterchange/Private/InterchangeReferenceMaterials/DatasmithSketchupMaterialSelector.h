// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"
#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialSelector.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		class FDatasmithSketchUpMaterialSelector : public FDatasmithReferenceMaterialSelector
		{
		public:
			FDatasmithSketchUpMaterialSelector();
			virtual ~FDatasmithSketchUpMaterialSelector() = default;

			virtual const TCHAR* GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const override;
#if WITH_EDITOR
			virtual void PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality MaterialQuality, UMaterialInstanceConstant* MaterialInstance) const override;
#endif
		};
	}
}