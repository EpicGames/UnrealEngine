// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithStdMaterialSelector.h"

#include "Misc/PackageName.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		namespace Standard
		{
			const TCHAR* OpaqueMaterialPath = TEXT("/DatasmithContent/Materials/StdOpaque/M_StdOpaque.M_StdOpaque");
			const TCHAR* TranslucentMaterialPath = TEXT("/DatasmithContent/Materials/StdTranslucent/M_StdTranslucent.M_StdTranslucent");
			const TCHAR* EmissiveMaterialPath = TEXT("/DatasmithContent/Materials/StdEmissive/M_StdEmissive.M_StdEmissive");
		}

		FDatasmithStdMaterialSelector::FDatasmithStdMaterialSelector()
		{
			bIsValid = FPackageName::DoesPackageExist(Standard::OpaqueMaterialPath) && FPackageName::DoesPackageExist(Standard::TranslucentMaterialPath) && FPackageName::DoesPackageExist(Standard::EmissiveMaterialPath);
		}

		const TCHAR* FDatasmithStdMaterialSelector::GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const
		{
			switch (MaterialType)
			{
			case EDatasmithReferenceMaterialType::Transparent:
				return Standard::TranslucentMaterialPath;
			case EDatasmithReferenceMaterialType::Emissive:
				return Standard::EmissiveMaterialPath;
			default:
				return Standard::OpaqueMaterialPath;
			}
		}
	}
}