// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "InterchangeReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "Templates/SharedPointer.h"

namespace UE
{
	namespace DatasmithInterchange
	{
		class FDatasmithStdMaterialSelector : public FDatasmithReferenceMaterialSelector
		{
		public:
			FDatasmithStdMaterialSelector();
			virtual ~FDatasmithStdMaterialSelector() = default;

			virtual const TCHAR* GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const override;
		};
	}
}
