// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()


#define DATASMITHINTERCHANGE_MODULE_NAME TEXT("DatasmithInterchange")

/**
 * The public interface of the DatasmithInterchange module
 */
class IDatasmithInterchangeModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to IDatasmithImporter
	 *
	 * @return Returns DatasmithImporter singleton instance, loading the module on demand if needed
	 */
	static inline IDatasmithInterchangeModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDatasmithInterchangeModule>(DATASMITHINTERCHANGE_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATASMITHINTERCHANGE_MODULE_NAME);
	}
};