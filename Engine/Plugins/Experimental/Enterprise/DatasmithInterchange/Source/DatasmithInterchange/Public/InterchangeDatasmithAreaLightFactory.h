// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Scene/InterchangeActorFactory.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeDatasmithAreaLightFactory.generated.h"


UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithAreaLightFactory : public UInterchangeActorFactory
{
	GENERATED_BODY()

public:
	//////////////////////////////////////////////////////////////////////////
	// Interchange factory base interface begin

	virtual UClass* GetFactoryClass() const override;

	// Interchange factory base interface end
	//////////////////////////////////////////////////////////////////////////

protected:
	//////////////////////////////////////////////////////////////////////////
	// Interchange actor factory interface begin

	virtual UObject* ProcessActor(class AActor& SpawnedActor, const UInterchangeActorFactoryNode& FactoryNode, const UInterchangeBaseNodeContainer& NodeContainer, const FImportSceneObjectsParams& Params) override;

	// Interchange actor factory interface end
	//////////////////////////////////////////////////////////////////////////
};