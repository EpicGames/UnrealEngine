// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeActorFactoryNode.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "DatasmithAreaLightActor.h"

#include "InterchangeDatasmithAreaLightFactoryNode.generated.h"

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithAreaLightFactoryNode : public UInterchangeActorFactoryNode
{
	GENERATED_BODY()

public:

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override;

	virtual UClass* GetObjectClass() const override;


	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomLightType(EDatasmithAreaLightActorType& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomLightType(const EDatasmithAreaLightActorType& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomLightShape(EDatasmithAreaLightActorShape& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomLightShape(const EDatasmithAreaLightActorShape& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomDimensions(FVector2D& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomDimensions(const FVector2D& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomIntensity(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomIntensity(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomIntensityUnits(ELightUnits& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomIntensityUnits(const ELightUnits& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomColor(FLinearColor& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomColor(const FLinearColor& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomTemperature(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomTemperature(const float& AttributeValue, bool bAddApplyDelegate = true);

	/**
	 * Get the NodeUid of the imported IES Texture.
	 */
	//UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomIESTexture(FString& AttributeValue) const;

	/**
	 * Set the NodeUid of the imported IES Texture.
	 */
	//UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomIESTexture(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomUseIESBrightness(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomUseIESBrightness(const bool& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomIESBrightnessScale(float& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomIESBrightnessScale(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomRotation(FRotator& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomRotation(const FRotator& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSourceRadius(float& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSourceRadius(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSourceLength(float& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSourceLength(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomAttenuationRadius(float& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomAttenuationRadius(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSpotlightInnerAngle(float& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSpotlightInnerAngle(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSpotlightOuterAngle(float& AttributeValue) const;
	
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSpotlightOuterAngle(const float& AttributeValue, bool bAddApplyDelegate = true);

	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object) override;

private:
	static  UE::Interchange::FAttributeKey Macro_CustomLightTypeKey;
	static  UE::Interchange::FAttributeKey Macro_CustomLightShapeKey;
	static  UE::Interchange::FAttributeKey Macro_CustomDimensionsKey;
	static  UE::Interchange::FAttributeKey Macro_CustomIntensityKey;
	static  UE::Interchange::FAttributeKey Macro_CustomIntensityUnitsKey;
	static  UE::Interchange::FAttributeKey Macro_CustomColorKey;
	static  UE::Interchange::FAttributeKey Macro_CustomTemperatureKey;
	static  UE::Interchange::FAttributeKey Macro_CustomIESTextureKey;
	static  UE::Interchange::FAttributeKey Macro_CustombUseIESBrightnessKey;
	static  UE::Interchange::FAttributeKey Macro_CustomIESBrightnessScaleKey;
	static  UE::Interchange::FAttributeKey Macro_CustomRotationKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSourceRadiusKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSourceLengthKey;
	static  UE::Interchange::FAttributeKey Macro_CustomAttenuationRadiusKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSpotlightInnerAngleKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSpotlightOuterAngleKey;
};