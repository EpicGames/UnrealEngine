// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/AttributeStorage.h"
#include "InterchangeLightNode.h"
#include "DatasmithAreaLightActor.h"

#include "InterchangeDatasmithAreaLightNode.generated.h"

namespace UE::DatasmithImporter
{
	class FExternalSource;
}

class IDatasmithScene;

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithAreaLightNode : public UInterchangeLightNode
{
	GENERATED_BODY()

public:
	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	virtual FString GetTypeName() const override
	{
		const FString TypeName = TEXT("DatasmithAreaLightNode");
		return TypeName;
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomLightType(EDatasmithAreaLightActorType& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomLightType(const EDatasmithAreaLightActorType& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomLightShape(EDatasmithAreaLightActorShape& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomLightShape(const EDatasmithAreaLightActorShape& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomDimensions(FVector2D& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomDimensions(const FVector2D& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomColor(FLinearColor& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomColor(const FLinearColor& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSourceRadius(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSourceRadius(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSourceLength(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSourceLength(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSpotlightInnerAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSpotlightInnerAngle(const float& AttributeValue, bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool GetCustomSpotlightOuterAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Datasmith")
	bool SetCustomSpotlightOuterAngle(const float& AttributeValue, bool bAddApplyDelegate = true);

private:
	static  UE::Interchange::FAttributeKey Macro_CustomLightTypeKey;
	static  UE::Interchange::FAttributeKey Macro_CustomLightShapeKey;
	static  UE::Interchange::FAttributeKey Macro_CustomDimensionsKey;
	static  UE::Interchange::FAttributeKey Macro_CustomColorKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSourceRadiusKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSourceLengthKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSpotlightInnerAngleKey;
	static  UE::Interchange::FAttributeKey Macro_CustomSpotlightOuterAngleKey;
};