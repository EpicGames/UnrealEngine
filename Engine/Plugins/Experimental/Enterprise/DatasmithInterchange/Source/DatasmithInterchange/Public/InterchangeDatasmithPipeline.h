// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangePipelineBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeDatasmithPipeline.generated.h"

class UDatasmithScene;
class UMaterialInterface;
class UInterchangeGenericAnimationPipeline;
class UInterchangeGenericCommonMeshesProperties;
class UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties;
class UInterchangeDatasmithLevelPipeline;
class UInterchangeDatasmithMaterialPipeline;
class UInterchangeDatasmithStaticMeshPipeline;
class UInterchangeDatasmithSceneNode;
class UInterchangeGenericTexturePipeline;
class UInterchangeTextureFactoryNode;
class UTexture;
struct FDatasmithImportContext;

UCLASS(BlueprintType, Experimental)
class DATASMITHINTERCHANGE_API UInterchangeDatasmithPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

	UInterchangeDatasmithPipeline();

	UPROPERTY(VisibleAnywhere, Instanced, Category = "Materials")
	TObjectPtr<UInterchangeDatasmithMaterialPipeline> MaterialPipeline;

	UPROPERTY(VisibleAnywhere, Instanced, Category = "Mesh")
	TObjectPtr<UInterchangeDatasmithStaticMeshPipeline> MeshPipeline;

	UPROPERTY(VisibleAnywhere, Instanced, Category = "Level")
	TObjectPtr<UInterchangeDatasmithLevelPipeline> LevelPipeline;

	UPROPERTY(VisibleAnywhere, Instanced, Category = "Animation")
	TObjectPtr<UInterchangeGenericAnimationPipeline> AnimationPipeline;

	UPROPERTY(VisibleAnywhere, Instanced, Category = "Common Meshes")
	TObjectPtr<UInterchangeGenericCommonMeshesProperties> CommonMeshesProperties;

	UPROPERTY(VisibleAnywhere, Instanced, Category = "Common Skeletal Meshes and Animations")
	TObjectPtr<UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties> CommonSkeletalMeshesAndAnimationsProperties;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Mesh")
	bool bDeleteInvalidMeshes = true;
#endif

	UPROPERTY(EditAnywhere, Category = "Materials")
	bool bCreateMaterialReferencesFolders = true;

protected:
	virtual void ExecutePipeline(UInterchangeBaseNodeContainer* InBaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath) override;
	virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* InBaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;
	virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams) override;

	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override
	{
		// This pipeline creates UObjects and assets. Not safe to execute outside of main thread.
		return false;
	}

private:
	void PreImportTextureFactoryNode(UInterchangeTextureFactoryNode* TextureFactoryNode) const;

private:
	UInterchangeBaseNodeContainer* BaseNodeContainer = nullptr;
};
