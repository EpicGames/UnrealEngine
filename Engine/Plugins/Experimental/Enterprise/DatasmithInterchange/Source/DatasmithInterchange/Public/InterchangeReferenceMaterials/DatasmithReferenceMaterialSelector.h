// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DatasmithDefinitions.h"

class UMaterialInstanceConstant;

namespace UE
{
	namespace DatasmithInterchange
	{
		class FDatasmithReferenceMaterial;

		class DATASMITHINTERCHANGE_API FDatasmithReferenceMaterialSelector
		{
		public:
			virtual ~FDatasmithReferenceMaterialSelector() = default;

			virtual bool IsValid() const { return bIsValid; }
			virtual const TCHAR* GetMaterialPath(EDatasmithReferenceMaterialType MaterialType) const { return nullptr; }
#if WITH_EDITOR
			virtual void PostImportProcess(EDatasmithReferenceMaterialType MaterialType, EDatasmithReferenceMaterialQuality MaterialQuality, UMaterialInstanceConstant* MaterialInstance) const {}
#endif

		protected:
			bool bIsValid = false;
		};
	}
}

